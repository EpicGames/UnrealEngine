// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FAchievementsGetCachedAchievementDescriptionStep : public FTestPipeline::FStep
{
	FAchievementsGetCachedAchievementDescriptionStep(const FString& InAchievementId, TFunction<void(FOnlineAchievementDesc*)>&& InStateSaver)
		: AchievementId(InAchievementId)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FAchievementsGetCachedAchievementDescriptionStep(const FString& InAchievementId)
		: AchievementId(InAchievementId)
		, StateSaver([](FOnlineAchievementDesc*) {})
	{}

	virtual ~FAchievementsGetCachedAchievementDescriptionStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineAchievementsPtr = OnlineSubsystem->GetAchievementsInterface();
		REQUIRE(OnlineAchievementsPtr != nullptr);

		EOnlineCachedResult::Type Result = OnlineAchievementsPtr->GetCachedAchievementDescription(AchievementId, OutAchievementDesc);
		CHECK(Result == EOnlineCachedResult::Type::Success);

		StateSaver(&OutAchievementDesc);

		return EContinuance::Done;
	}

protected:
	IOnlineAchievementsPtr OnlineAchievementsPtr = nullptr;
	FString AchievementId = FString(TEXT(""));
	FOnlineAchievementDesc OutAchievementDesc;
	TFunction<void(FOnlineAchievementDesc*)> StateSaver;

};