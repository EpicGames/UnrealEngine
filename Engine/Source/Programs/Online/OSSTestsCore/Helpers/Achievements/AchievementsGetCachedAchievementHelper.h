// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FAchievementsGetCachedAchievementStep : public FTestPipeline::FStep
{
	FAchievementsGetCachedAchievementStep(FUniqueNetIdPtr* InPlayerId, const FString& InAchievementId, TFunction<void(FOnlineAchievement*)>&& InStateSaver)
		: PlayerId(InPlayerId)
		, AchievementId(InAchievementId)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FAchievementsGetCachedAchievementStep(FUniqueNetIdPtr* InPlayerId, const FString& InAchievementId)
		: PlayerId(InPlayerId)
		, AchievementId(InAchievementId)
		, StateSaver([](FOnlineAchievement*) {})
	{}

	virtual ~FAchievementsGetCachedAchievementStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineAchievementsPtr = OnlineSubsystem->GetAchievementsInterface();
		REQUIRE(OnlineAchievementsPtr != nullptr);

		EOnlineCachedResult::Type Result = OnlineAchievementsPtr->GetCachedAchievement(*PlayerId->Get(), AchievementId, OutAchievement);
		CHECK(Result == EOnlineCachedResult::Type::Success);

		StateSaver(&OutAchievement);

		return EContinuance::Done;
	}

protected:
	IOnlineAchievementsPtr OnlineAchievementsPtr = nullptr;
	FUniqueNetIdPtr* PlayerId = nullptr;
	FString AchievementId = FString(TEXT(""));
	FOnlineAchievement OutAchievement;
	TFunction<void(FOnlineAchievement*)> StateSaver;

};