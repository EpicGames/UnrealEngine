// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FAchievementsGetCachedAchievementsStep : public FTestPipeline::FStep
{
	FAchievementsGetCachedAchievementsStep(FUniqueNetIdPtr* InPlayerId, TFunction<void(TArray<FOnlineAchievement>*)>&& InStateSaver)
		: PlayerId(InPlayerId)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FAchievementsGetCachedAchievementsStep(FUniqueNetIdPtr* InPlayerId)
		: PlayerId(InPlayerId)
		, StateSaver([](TArray<FOnlineAchievement>*) {})
	{}

	virtual ~FAchievementsGetCachedAchievementsStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineAchievementsPtr = OnlineSubsystem->GetAchievementsInterface();
		REQUIRE(OnlineAchievementsPtr != nullptr);
		
		EOnlineCachedResult::Type Result = OnlineAchievementsPtr->GetCachedAchievements(*PlayerId->Get(), OutAchievements);
		CHECK(Result == EOnlineCachedResult::Type::Success);

		StateSaver(&OutAchievements);

		return EContinuance::Done;
	}

protected:
	IOnlineAchievementsPtr OnlineAchievementsPtr = nullptr;
	FUniqueNetIdPtr* PlayerId = nullptr;
	TArray<FOnlineAchievement> OutAchievements;
	TFunction<void(TArray<FOnlineAchievement>*)> StateSaver;

};