// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystem.h"

struct FAchievementsQueryAchievementDescriptionsStep : public FTestPipeline::FStep
{
	FAchievementsQueryAchievementDescriptionsStep(FUniqueNetIdPtr* InPlayerId)
		: PlayerId(InPlayerId)
	{}

	virtual ~FAchievementsQueryAchievementDescriptionsStep() = default;

	enum class EState { Init, QueryAchievementDescriptionsCall, QueryAchievementDescriptionsCalled, Done } State = EState::Init;

	FOnQueryAchievementsCompleteDelegate OnQueryAchievementsCompleteDelegate = FOnQueryAchievementsCompleteDelegate::CreateLambda([this](const FUniqueNetId& QueryAchievementDescriptionsPlayerId, const bool bQueryAchievementDescriptionsWasSuccessful)
		{
			CHECK(State == EState::QueryAchievementDescriptionsCalled);
			CHECK(bQueryAchievementDescriptionsWasSuccessful);
			CHECK(QueryAchievementDescriptionsPlayerId == *PlayerId->Get());

			State = EState::Done;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineAchievementsPtr = OnlineSubsystem->GetAchievementsInterface();
				REQUIRE(OnlineAchievementsPtr != nullptr);
				State = EState::QueryAchievementDescriptionsCall;

				break;
			}
			case EState::QueryAchievementDescriptionsCall:
			{
				State = EState::QueryAchievementDescriptionsCalled;

				OnlineAchievementsPtr->QueryAchievementDescriptions(*PlayerId->Get(), OnQueryAchievementsCompleteDelegate);

				break;
			}
			case EState::QueryAchievementDescriptionsCalled:
			{
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineAchievementsPtr OnlineAchievementsPtr = nullptr;
	FUniqueNetIdPtr* PlayerId = nullptr;
};