// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystem.h"

struct FAchievementsQueryAchievementsStep : public FTestPipeline::FStep
{
	FAchievementsQueryAchievementsStep(FUniqueNetIdPtr* InPlayerId)
		: PlayerId(InPlayerId)
	{}

	virtual ~FAchievementsQueryAchievementsStep() = default;

	enum class EState { Init, QueryAchievementsCall, QueryAchievementsCalled, Done } State = EState::Init;

	FOnQueryAchievementsCompleteDelegate OnQueryAchievementsCompleteDelegate = FOnQueryAchievementsCompleteDelegate::CreateLambda([this](const FUniqueNetId& SecondQueryAchievementsPlayerId, const bool bSecondQueryAchievementsWasSuccessful)
		{
			CHECK(State == EState::QueryAchievementsCalled);
			CHECK(bSecondQueryAchievementsWasSuccessful);
			CHECK(SecondQueryAchievementsPlayerId == *PlayerId->Get());

			State = EState::Done;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineAchievementsPtr = OnlineSubsystem->GetAchievementsInterface();
				REQUIRE(OnlineAchievementsPtr != nullptr);

				State = EState::QueryAchievementsCall;

				break;
			}
			case EState::QueryAchievementsCall:
			{
				State = EState::QueryAchievementsCalled;

				OnlineAchievementsPtr->QueryAchievements(*PlayerId->Get(), OnQueryAchievementsCompleteDelegate);

				break;
			}
			case EState::QueryAchievementsCalled:
			{
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineAchievementsPtr OnlineAchievementsPtr = nullptr;
	FUniqueNetIdPtr* PlayerId = nullptr;
};