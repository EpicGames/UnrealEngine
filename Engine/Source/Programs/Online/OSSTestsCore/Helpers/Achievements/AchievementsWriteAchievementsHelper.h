// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineAchievementsInterface.h"
#include "OnlineSubsystem.h"

struct FAchievementsWriteAchievementsStep : public FTestPipeline::FStep
{
	FAchievementsWriteAchievementsStep(FUniqueNetIdPtr* InPlayerId, FOnlineAchievementsWriteRef& InWriteObject)
		: PlayerId(InPlayerId)
		, WriteObject(InWriteObject)
	{}

	virtual ~FAchievementsWriteAchievementsStep() = default;

	enum class EState { Init, WriteAchievementsCall, WriteAchievementsCalled, Done } State = EState::Init;

	FOnAchievementsWrittenDelegate OnOnAchievementsWrittenDelegate = FOnAchievementsWrittenDelegate::CreateLambda([this](const FUniqueNetId& WriteAchievementsPlayerId, bool bWriteAchievementsWasSuccessful)
		{
			CHECK(State == EState::WriteAchievementsCalled);
			CHECK(bWriteAchievementsWasSuccessful);
			CHECK(WriteAchievementsPlayerId == *PlayerId->Get());

			State = EState::Done;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineAchievementsPtr = OnlineSubsystem->GetAchievementsInterface();
				REQUIRE(OnlineAchievementsPtr != nullptr);

				State = EState::WriteAchievementsCall;

				break;
			}
			case EState::WriteAchievementsCall:
			{
				State = EState::WriteAchievementsCalled;

				OnlineAchievementsPtr->WriteAchievements(*PlayerId->Get(), WriteObject, OnOnAchievementsWrittenDelegate);

				break;
			}
			case EState::WriteAchievementsCalled:
			{
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineAchievementsPtr OnlineAchievementsPtr = nullptr;
	FUniqueNetIdPtr* PlayerId = nullptr;
	FOnlineAchievementsWriteRef& WriteObject;
};