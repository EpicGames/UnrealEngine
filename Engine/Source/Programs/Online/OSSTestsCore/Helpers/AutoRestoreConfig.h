// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

template<typename T>
class TAutoRestoreGConfig
{
public:
	TAutoRestoreGConfig(const TCHAR* InSectionName, const TCHAR* InKeyName, const FString& InFileName)
		: SectionName(InSectionName), KeyName(InKeyName), FileName(InFileName)
	{
	}

	~TAutoRestoreGConfig()
	{
		Reset();
	}

	void Reset()
	{
		if (bRetrievedInitialValue)
		{
			if (InitialValue.IsSet())
			{
				SetValue(MoveTempIfPossible(InitialValue.GetValue()));
				InitialValue.Reset();
			}
			else
			{
				FConfigFile* ConfigFile = GConfig->FindConfigFile(FileName);
				CHECK(ConfigFile);
				ConfigFile->RemoveKeyFromSection(SectionName, KeyName);
			}
			bRetrievedInitialValue = false;
		}
	}

	bool IsSet()
	{
		return bRetrievedInitialValue;
	}

	void SetValue(const T& NewValue)
	{
		if (!bRetrievedInitialValue)
		{
			T PrevValue = {};
			if (GConfig->GetValue(SectionName, KeyName, PrevValue, FileName))
			{
				InitialValue.Emplace(PrevValue);
			}
			bRetrievedInitialValue = true;
		}

		if constexpr (std::is_same_v<T, FString>)
		{
			GConfig->SetString(SectionName, KeyName, *NewValue, FileName);
		}
		else if constexpr (std::is_same_v<T, bool>)
		{
			GConfig->SetBool(SectionName, KeyName, NewValue, FileName);
		}
		else
		{
			checkNoEntry();
		}
	}

private:
	TOptional<T> InitialValue;
	bool bRetrievedInitialValue = false;
	const TCHAR* SectionName;
	const TCHAR* KeyName;
	const FString& FileName;
};
