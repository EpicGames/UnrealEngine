// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsAcceptInviteStep : public FTestPipeline::FStep
{
	FFriendsAcceptInviteStep(int32 InUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InListName)
		: UserNum(InUserNum)
		, UserId(InLocalUserId)
		, ListName(InListName)
	{}

	virtual ~FFriendsAcceptInviteStep() = default;

	enum class EState { Init, AcceptInviteCalled, ClearDelegates, Done } State = EState::Init;

	FOnAcceptInviteComplete AcceptInvite = FOnAcceptInviteComplete::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FUniqueNetId& InUserId, const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::AcceptInviteCalled);
			CHECK(InUserNum == UserNum);
			CHECK(bInWasSuccessful);
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == ListName);
			CHECK(InErrorStr.Len() == 0);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
 			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();

			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::AcceptInviteCalled;
			bool Result = OnlineFriendsPtr->AcceptInvite(UserNum, *UserId->Get(), ListName, AcceptInvite);

			CHECK(Result == true);
			break;
		}
		case EState::AcceptInviteCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
