// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsBlockPlayerStep : public FTestPipeline::FStep
{
	FFriendsBlockPlayerStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
	{}

	virtual ~FFriendsBlockPlayerStep() = default;

	enum class EState { Init, BlockPlayerCall, BlockPlayerCalled, ClearDelegates, Done } State = EState::Init;

	FOnBlockedPlayerCompleteDelegate BlockPlayer = FOnBlockedPlayerCompleteDelegate::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FUniqueNetId& InUserId,  const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::BlockPlayerCalled);
			CHECK(InUserNum == UserNum);
			CHECK(bInWasSuccessful);
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == TEXT("BlockedPlayers"));
			CHECK(InErrorStr.Len() == 0);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();

		switch (State)
		{
		case EState::Init:
		{
			OnBlockedPlayerCompleteDelegateHandle = OnlineFriendsPtr->AddOnBlockedPlayerCompleteDelegate_Handle(UserNum, BlockPlayer);
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::BlockPlayerCall;
			break;
		}
		case EState::BlockPlayerCall:
		{
			State = EState::BlockPlayerCalled;
			bool Result = OnlineFriendsPtr->BlockPlayer(UserNum, *UserId->Get());

			CHECK(Result == true);
			break;
		}
		case EState::BlockPlayerCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineFriendsPtr->ClearOnBlockedPlayerCompleteDelegate_Handle(UserNum, OnBlockedPlayerCompleteDelegateHandle);
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	FDelegateHandle	OnBlockedPlayerCompleteDelegateHandle;
};
