// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineError.h"
#include "OnlineSubsystem.h"

struct FFriendsDeleteFriendAliasStep : public FTestPipeline::FStep
{
	FFriendsDeleteFriendAliasStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsDeleteFriendAliasStep() = default;

	enum class EState { Init, DeleteFriendAliasCall, DeleteFriendAliasCalled, ClearDelegates, Done } State = EState::Init;

	FOnDeleteFriendAliasComplete DeleteFriendAlias = FOnDeleteFriendAliasComplete::CreateLambda([this](int32 InUserNum, const FUniqueNetId& InUserId, const FString& InListName, const FOnlineError& InError)
		{
			CHECK(State == EState::DeleteFriendAliasCalled);
			CHECK(InUserNum == UserNum);
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == ListName);
			CHECK(InError == FOnlineError::Success());
			
			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::DeleteFriendAliasCall;
			break;
		}
		case EState::DeleteFriendAliasCall:
		{
			State = EState::DeleteFriendAliasCalled;
			OnlineFriendsPtr->DeleteFriendAlias(UserNum, *UserId->Get(), ListName, DeleteFriendAlias);

			break;
		}
		case EState::DeleteFriendAliasCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	TArray<FUniqueNetIdRef> Players;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
