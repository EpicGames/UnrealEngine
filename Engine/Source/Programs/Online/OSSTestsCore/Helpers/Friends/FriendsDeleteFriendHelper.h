// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsDeleteFriendStep : public FTestPipeline::FStep
{
	FFriendsDeleteFriendStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsDeleteFriendStep()
	{
		if (OnlineFriendsPtr != nullptr)
		{
			if (OnlineFriendsPtr->OnDeleteFriendCompleteDelegates->IsBound())
			{
				OnlineFriendsPtr->OnDeleteFriendCompleteDelegates->Clear();
			}
			OnlineFriendsPtr = nullptr;
		}
	};

	enum class EState { Init, DeleteFriendCall, DeleteFriendCalled, ClearDelegates, Done } State = EState::Init;

	FOnDeleteFriendCompleteDelegate DeleteFriend = FOnDeleteFriendCompleteDelegate::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FUniqueNetId& InUserId, const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::DeleteFriendCalled);
			CHECK(InUserNum == UserNum);

			//TODO: At this point, we ignore the checking for this parameter because we want the joint tests to work correctly
			/*CHECK(bInWasSuccessful);*/

			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == ListName);

			FString FriendNotFoundError = FString::Printf(TEXT("errors.com.epicgames.oss.friend.friend_not_found:%s"), *InUserId.ToString());
			bool IsErrorDoesNotExists = InErrorStr == FriendNotFoundError ? true : InErrorStr.Len() == 0;

			CHECK(IsErrorDoesNotExists);

			State = EState::ClearDelegates;

		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();

		switch (State)
		{
		case EState::Init:
		{
			OnDeleteFriendCompleteDelegateHandle = OnlineFriendsPtr->AddOnDeleteFriendCompleteDelegate_Handle(UserNum, DeleteFriend);

			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::DeleteFriendCalled;

			//TODO: At this point, we ignore the checking for this parameter because we want the joint tests to work correctly
			/*bool Result = */
		    OnlineFriendsPtr->DeleteFriend(UserNum, *UserId->Get(), ListName);

			/*CHECK(Result == true);*/
			break;
		}
		case EState::DeleteFriendCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineFriendsPtr->ClearOnDeleteFriendCompleteDelegate_Handle(UserNum, OnDeleteFriendCompleteDelegateHandle);
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	FDelegateHandle	OnDeleteFriendCompleteDelegateHandle;
};
