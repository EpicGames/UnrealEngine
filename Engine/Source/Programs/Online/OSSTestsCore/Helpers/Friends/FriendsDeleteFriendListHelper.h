// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsDeleteFriendListStep : public FTestPipeline::FStep
{
	FFriendsDeleteFriendListStep(int32 InLocalUserNum, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsDeleteFriendListStep() = default;

	enum class EState { Init, DeleteFriendListCall, DeleteFriendListCalled, ClearDelegates, Done } State = EState::Init;

	FOnReadFriendsListComplete DeleteFriendsList = FOnDeleteFriendsListComplete::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::DeleteFriendListCalled);
			CHECK(InUserNum == UserNum);
			CHECK(bInWasSuccessful);
			CHECK(InListName == ListName);
			CHECK(InErrorStr.Len() == 0);

			TArray<TSharedRef<FOnlineFriend>> FriendsList;
			OnlineFriendsPtr->GetFriendsList(UserNum, ListName, FriendsList);

			CHECK(FriendsList.Num() == 0);
			
			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::DeleteFriendListCall;
			break;
		}
		case EState::DeleteFriendListCall:
		{
			State = EState::DeleteFriendListCalled;
			bool Result = OnlineFriendsPtr->DeleteFriendsList(UserNum, ListName, DeleteFriendsList);

			CHECK(Result == true);
			break;
		}
		case EState::DeleteFriendListCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
