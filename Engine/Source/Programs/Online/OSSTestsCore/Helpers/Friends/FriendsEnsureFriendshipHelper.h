// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "OnlineSubsystem.h"
#include "Interfaces/OnlineFriendsInterface.h"

#include "Helpers/Friends/FriendsDeleteFriendListHelper.h"
#include "Helpers/Friends/FriendsReadFriendsListHelper.h"
#include "Helpers/Friends/FriendsRejectInviteHelper.h"
#include "Helpers/Friends/FriendsSendInviteHelper.h"
#include "Helpers/Friends/FriendsAcceptInviteHelper.h"
#include "Helpers/Friends/FriendsIsFriendHelper.h"
#include "Helpers/Friends/FriendsUnblockPlayerHelper.h"

#include "Helpers/InnerStepBuilder.h"

struct FFriendsEnsureFriendshipStep : public FTestPipeline::FStep
{
	FFriendsEnsureFriendshipStep(int32 InLocalUserNum, int32 InTargetUserNum, FUniqueNetIdPtr* InLocalUserId, FUniqueNetIdPtr* InTargetUserId, FString& InLocalListName, bool InbIsFriendsListPopulated = false)
		: InnerSteps(
			TInnerStepArrayBuilder<FTestPipeline::FStep>()
			.EmplaceInnerStep<FFriendsDeleteFriendListStep>(InLocalUserNum, InLocalListName)
			.EmplaceInnerStep<FFriendsDeleteFriendListStep>(InTargetUserNum, InLocalListName)
			.EmplaceInnerStep<FFriendsUnblockPlayerStep>(InLocalUserNum, InTargetUserId)
			.EmplaceInnerStep<FFriendsRejectInviteStep>(InTargetUserNum, InLocalUserId, InLocalListName)
			.EmplaceInnerStep<FFriendsSendInviteStep>(InLocalUserNum, InTargetUserId, InLocalListName)
			.EmplaceInnerStep<FFriendsReadFriendsListStep>(InTargetUserNum, InLocalListName)
			.EmplaceInnerStep<FFriendsAcceptInviteStep>(InTargetUserNum, InLocalUserId, InLocalListName)
			.EmplaceInnerStep<FFriendsReadFriendsListStep>(InTargetUserNum, InLocalListName, InbIsFriendsListPopulated)
			.EmplaceInnerStep<FFriendsIsFriendStep>(InTargetUserNum, InLocalUserId, InLocalListName)
			.Steps)
		, OldVerbosity(LogOnlineFriend.GetVerbosity())
		, CurrentVerbosity([]()
			{
				LogOnlineFriend.SetVerbosity(ELogVerbosity::Error);
				return ELogVerbosity::Error;
			}())
	{}

	virtual ~FFriendsEnsureFriendshipStep()
	{
		if (OldVerbosity != LogOnlineFriend.GetVerbosity())
		{
			LogOnlineFriend.SetVerbosity(OldVerbosity);
		}
	}

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		if (InnerSteps.Num() == 0)
		{
			return EContinuance::Done;
		}

		OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
		REQUIRE(OnlineFriendsPtr != nullptr);

		CurrentStepContinuanceResult = InnerSteps.HeapTop()->Tick(OnlineSubsystem);

		if (CurrentStepContinuanceResult == FStep::EContinuance::Done)
		{
			TCheckedPointerIterator<FTestPipeline::FStepPtr, int32> It = InnerSteps.begin();
			InnerSteps.RemoveSingle(*It);
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FStep::EContinuance CurrentStepContinuanceResult = FStep::EContinuance::ContinueStepping;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	TArray<TUniquePtr<FTestPipeline::FStep>> InnerSteps;
	ELogVerbosity::Type OldVerbosity;
	ELogVerbosity::Type CurrentVerbosity;
};