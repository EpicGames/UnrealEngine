// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsGetBlockedPlayersStep : public FTestPipeline::FStep
{
	FFriendsGetBlockedPlayersStep(FUniqueNetIdPtr* InLocalUserId, FUniqueNetIdPtr* InTargetUserId = nullptr)
		: UserId(InLocalUserId)
		, TargetUserId(InTargetUserId)
	{}

	virtual ~FFriendsGetBlockedPlayersStep() = default;

	enum class EState { Init, GetBlockedPlayersCall, GetBlockedPlayersCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::GetBlockedPlayersCall;
			break;
		}
		case EState::GetBlockedPlayersCall:
		{
			State = EState::GetBlockedPlayersCalled;
			bool Result = OnlineFriendsPtr->GetBlockedPlayers(*UserId->Get(), OutBlockedPlayers);

			if (TargetUserId != nullptr)
			{
				bool bFoundBlockedPlayer = false;
				
				for (TSharedRef<FOnlineBlockedPlayer> BlockedPlayer : OutBlockedPlayers)
				{
					if (BlockedPlayer->GetUserId()->ToString() == *TargetUserId->Get()->ToString())
					{
						bFoundBlockedPlayer = true;
						break;
					}
				}

				CHECK(bFoundBlockedPlayer);
			}
		
			CHECK(Result == true);
			break;
		}
		case EState::GetBlockedPlayersCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	FUniqueNetIdPtr* TargetUserId = nullptr;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	TArray<TSharedRef<FOnlineBlockedPlayer>> OutBlockedPlayers;
};
