// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "OnlineSubsystem.h"

struct FFriendsGetFriendStep : public FTestPipeline::FStep
{
	FFriendsGetFriendStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsGetFriendStep() = default;

	enum class EState { Init, GetFriendCall, GetFriendCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{

		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::GetFriendCall;
			break;
		}
		case EState::GetFriendCall:
		{
			State = EState::GetFriendCalled;
			TSharedPtr<FOnlineFriend> FriendEntry = OnlineFriendsPtr->GetFriend(UserNum, *UserId->Get(), ListName);

			REQUIRE(FriendEntry != nullptr);

			if (FriendEntry.IsValid())
			{
				CHECK(*FriendEntry->GetUserId() == *UserId->Get());
			}

			break;
		}
		case EState::GetFriendCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
