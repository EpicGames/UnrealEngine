// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "OnlineSubsystem.h"

struct FFriendsGetFriendInvitePolicyStep : public FTestPipeline::FStep
{
	FFriendsGetFriendInvitePolicyStep(FUniqueNetIdPtr* InLocalUserId, const EFriendInvitePolicy& InLocalExpectedInvitePolicy)
		: UserId(InLocalUserId)
		, ExpectedInvitePolicy(InLocalExpectedInvitePolicy)
	{}

	virtual ~FFriendsGetFriendInvitePolicyStep() = default;

	enum class EState { Init, GetFriendInvitePolicyCall, GetFriendInvitePolicyCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{

		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::GetFriendInvitePolicyCall;
			break;
		}
		case EState::GetFriendInvitePolicyCall:
		{
			State = EState::GetFriendInvitePolicyCalled;
			EFriendInvitePolicy Result = OnlineFriendsPtr->GetFriendInvitePolicy(*UserId->Get());

			CHECK(Result == ExpectedInvitePolicy);
			break;
		}
		case EState::GetFriendInvitePolicyCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	EFriendInvitePolicy ExpectedInvitePolicy = EFriendInvitePolicy::InvalidOrMax;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
