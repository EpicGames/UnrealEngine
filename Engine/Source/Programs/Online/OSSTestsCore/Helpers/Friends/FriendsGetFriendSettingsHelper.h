// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsGetFriendSettingsStep : public FTestPipeline::FStep
{
	FFriendsGetFriendSettingsStep(FUniqueNetIdPtr* InLocalUserId, const FString& InLocalSource)
		: UserId(InLocalUserId)
		, Source(InLocalSource)
	{}

	virtual ~FFriendsGetFriendSettingsStep() = default;

	enum class EState { Init, GetFriendSettingsCall, GetFriendSettingsCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::GetFriendSettingsCall;
			break;
		}
		case EState::GetFriendSettingsCall:
		{
			State = EState::GetFriendSettingsCalled;
			bool Result = OnlineFriendsPtr->GetFriendSettings(*UserId->Get(), OutSettings);

			CHECK(Result == true);
			CHECK(OutSettings.Contains(Source));
			break;
		}
		case EState::GetFriendSettingsCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	FString Source;
	TMap<FString, TSharedRef<FOnlineFriendSettingsSourceData>> OutSettings;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
