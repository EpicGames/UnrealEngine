// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsGetFriendsListStep : public FTestPipeline::FStep
{
	FFriendsGetFriendsListStep(int32 InLocalUserNum, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsGetFriendsListStep()
	{
		if (OnlineFriendsPtr != nullptr)
		{
			if (OnlineFriendsPtr->OnRejectInviteCompleteDelegates->IsBound())
			{
				OnlineFriendsPtr->OnRejectInviteCompleteDelegates->Clear();
			}
			OnlineFriendsPtr = nullptr;
		}
	};

	enum class EState { Init, GetFriendsListCall, GetFriendsListCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
	
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::GetFriendsListCall;
			break;
		}
		case EState::GetFriendsListCall:
		{
			State = EState::GetFriendsListCalled;
			bool Result = OnlineFriendsPtr->GetFriendsList(UserNum, ListName, OutFriends);

			CHECK(OutFriends.Num() > 0);
			CHECK(Result == true);
			break;
		}
		case EState::GetFriendsListCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	TArray<TSharedRef<FOnlineFriend>> OutFriends;
};
