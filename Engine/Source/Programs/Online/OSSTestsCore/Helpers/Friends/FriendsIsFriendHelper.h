// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsIsFriendStep : public FTestPipeline::FStep
{
	FFriendsIsFriendStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsIsFriendStep() = default;

	enum class EState { Init, IsFriendCall, IsFriendCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::IsFriendCalled;
			bool Result = OnlineFriendsPtr->IsFriend(UserNum, *UserId->Get(), ListName);
			
			CHECK(Result == true);
			break;
		}
		case EState::IsFriendCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
