// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsQueryBlockedPlayersStep : public FTestPipeline::FStep
{
	FFriendsQueryBlockedPlayersStep(FUniqueNetIdPtr* InLocalUserId, bool InLocalIsBlockedPlayersListPopulated = false)
		: UserId(InLocalUserId)
		, IsBlockedPlayersListPopulated(InLocalIsBlockedPlayersListPopulated)
	{}

	virtual ~FFriendsQueryBlockedPlayersStep() = default;

	enum class EState { Init, QueryBlockedPlayersCall, QueryBlockedPlayersCalled, ClearDelegates, Done } State = EState::Init;

	FOnQueryBlockedPlayersCompleteDelegate QueryBlockedPlayers = FOnQueryBlockedPlayersCompleteDelegate::CreateLambda([this](const FUniqueNetId& InUserId, bool bInWasSuccessful, const FString& InErrorStr)
		{
			CHECK(State == EState::QueryBlockedPlayersCalled);
			CHECK(InUserId == *UserId->Get());
			CHECK(InErrorStr.Len() == 0);

			if (IsBlockedPlayersListPopulated)
			{
				TArray<TSharedRef<FOnlineBlockedPlayer>> BlockedPlayersList;
				OnlineFriendsPtr->GetBlockedPlayers(InUserId, BlockedPlayersList);

				CHECK(BlockedPlayersList.Num() > 0);
			}

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
		
		switch (State)
		{
		case EState::Init:
		{
			OnQueryBlockedPlayersCompleteDelegateHandle = OnlineFriendsPtr->AddOnQueryBlockedPlayersCompleteDelegate_Handle(QueryBlockedPlayers);
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::QueryBlockedPlayersCall;
			break;
		}
		case EState::QueryBlockedPlayersCall:
		{
			State = EState::QueryBlockedPlayersCalled;
			bool Result = OnlineFriendsPtr->QueryBlockedPlayers(*UserId->Get());

			CHECK(Result == true);
			break;
		}
		case EState::QueryBlockedPlayersCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineFriendsPtr->ClearOnQueryBlockedPlayersCompleteDelegate_Handle(OnQueryBlockedPlayersCompleteDelegateHandle);
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	bool IsBlockedPlayersListPopulated = false;
	FDelegateHandle	OnQueryBlockedPlayersCompleteDelegateHandle;
};
