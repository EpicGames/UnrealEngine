// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsQueryFriendSettingsStep : public FTestPipeline::FStep
{
	FFriendsQueryFriendSettingsStep(FUniqueNetIdPtr* InLocalUserId, const FString& InLocalSource)
		: UserId(InLocalUserId)
		, Source(InLocalSource)
	{}

	FFriendsQueryFriendSettingsStep(FUniqueNetIdPtr* InLocalUserId, FFriendSettings& InLocalNewSettings)
		: UserId(InLocalUserId)
		, ExpectedSettings(InLocalNewSettings)
		, bUseOverload(true)
	{}

	virtual ~FFriendsQueryFriendSettingsStep() = default;

	enum class EState { Init, QueryFriendSettingsCall, QueryFriendSettingsCalled, ClearDelegates, Done } State = EState::Init;

	FOnQueryFriendSettingsComplete OnQueryFriendSettings = FOnQueryFriendSettingsComplete::CreateLambda([this](const FUniqueNetId& InUserId, bool bInWasSuccessful, const FString& InErrorStr)
		{
			CHECK(State == EState::QueryFriendSettingsCalled);
			CHECK(InUserId == *UserId->Get());
			CHECK(bInWasSuccessful);
			CHECK(InErrorStr.Len() == 0);

			State = EState::ClearDelegates;
		});

	FOnSettingsOperationComplete OnSettingsOperation = FOnSettingsOperationComplete::CreateLambda([this](const FUniqueNetId& InUserId, bool bInWasSuccessful, bool bInWasUpdate, const FFriendSettings& InSettings, const FString& InErrorStr)
		{
			CHECK(State == EState::QueryFriendSettingsCalled);
			CHECK(InUserId == *UserId->Get());
			CHECK(bInWasSuccessful);

			//Because the operation can be a passive read
			//CHECK(bInWasUpdate);

			//At the moment we cannot get a correct value for this parameter. Ticket on Jira: OI-3541
			//bool bWasSuccessful = ExpectedSettings.SettingsMap.OrderIndependentCompareEqual(InSettings.SettingsMap);
			//CHECK(bWasSuccessful); //Should be true

			CHECK(InErrorStr.Len() == 0);
			
			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::QueryFriendSettingsCall;
			break;
		}
		case EState::QueryFriendSettingsCall: 
		{
			State = EState::QueryFriendSettingsCalled;

			if (bUseOverload)
			{
				OnlineFriendsPtr->QueryFriendSettings(*UserId->Get(), OnSettingsOperation);
			}
			else
			{
				bool Result = OnlineFriendsPtr->QueryFriendSettings(*UserId->Get(), Source, OnQueryFriendSettings);
				CHECK(Result == true);
			}
			
			break;
		}
		case EState::QueryFriendSettingsCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	FString Source;
	FFriendSettings ExpectedSettings;
	bool bUseOverload = false;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};

