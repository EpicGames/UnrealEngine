// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsReadFriendsListStep : public FTestPipeline::FStep
{
	FFriendsReadFriendsListStep(int32 InLocalUserNum, FString& InLocalListName, bool bInIsFriendsListPopulated = false)
		: UserNum(InLocalUserNum)
		, ListName(InLocalListName)
		, bIsFriendsListPopulated(bInIsFriendsListPopulated)
	{}

	virtual ~FFriendsReadFriendsListStep() = default;

	enum class EState { Init, ReadFriendsListCall, ReadFriendsListCalled, ClearDelegates, Done } State = EState::Init;

	FOnReadFriendsListComplete ReadFriendsList = FOnReadFriendsListComplete::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FString& InListName, const FString& InErrorStr)
		{		
			CHECK(State == EState::ReadFriendsListCalled);
			CHECK(InUserNum == UserNum);
			CHECK(bInWasSuccessful);
			CHECK(InListName == ListName);
			CHECK(InErrorStr.Len() == 0);
			
			if (bIsFriendsListPopulated)
			{
				TArray<TSharedRef<FOnlineFriend>> FriendsList;
				OnlineFriendsPtr->GetFriendsList(UserNum, ListName, FriendsList);
				CHECK(FriendsList.Num() > 0);
			}
			
 			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			//	State = EState::ReadFriendsListCall;
			//break;
			//}
			//case EState::ReadFriendsListCall:
			//{
			State = EState::ReadFriendsListCalled;
			bool Result = OnlineFriendsPtr->ReadFriendsList(UserNum, ListName, ReadFriendsList);
			
			CHECK(Result == true);
			break;
		}
		case EState::ReadFriendsListCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FString ListName;
	bool bIsFriendsListPopulated = false;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
