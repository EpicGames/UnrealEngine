// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsRejectInviteStep : public FTestPipeline::FStep
{
	FFriendsRejectInviteStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsRejectInviteStep()
	{
		if (OnlineFriendsPtr != nullptr)
		{
			if (OnlineFriendsPtr->OnRejectInviteCompleteDelegates->IsBound())
			{
				OnlineFriendsPtr->OnRejectInviteCompleteDelegates->Clear();
			}
			OnlineFriendsPtr = nullptr;
		}
	};

	enum class EState { Init, RejectInviteCall, RejectInviteCalled, ClearDelegates, Done } State = EState::Init;

	FOnRejectInviteCompleteDelegate RejectInvite = FOnRejectInviteCompleteDelegate::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FUniqueNetId& InUserId, const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::RejectInviteCalled);
			CHECK(InUserNum == UserNum);

			//TODO: At this point, we ignore the checking for this parameter because we want the joint tests to work correctly
			/*CHECK(bInWasSuccessful);*/
			
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == ListName);
		
			FString FriendNotFoundError = FString::Printf(TEXT("errors.com.epicgames.oss.friend.friend_not_found:%s"), *InUserId.ToString());
			bool IsErrorDoesNotExists = InErrorStr == FriendNotFoundError ? true : InErrorStr.Len() == 0;
	
			CHECK(IsErrorDoesNotExists);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();

		switch (State)
		{
		case EState::Init:
		{
			OnRejectInviteCompleteDelegateHandle = OnlineFriendsPtr->AddOnRejectInviteCompleteDelegate_Handle(UserNum, RejectInvite);
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::RejectInviteCalled;
			
			//TODO: At this point, we ignore the checking for this parameter because we want the joint tests to work correctly
			/*bool Result = */ 
			OnlineFriendsPtr->RejectInvite(UserNum, *UserId->Get(), ListName);

			/*CHECK(Result == true);*/
			break;
		}
		case EState::RejectInviteCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineFriendsPtr->ClearOnRejectInviteCompleteDelegate_Handle(UserNum, OnRejectInviteCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	FDelegateHandle	OnRejectInviteCompleteDelegateHandle;
};
