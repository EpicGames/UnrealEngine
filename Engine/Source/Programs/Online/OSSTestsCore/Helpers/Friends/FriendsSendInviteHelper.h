// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsSendInviteStep : public FTestPipeline::FStep
{
	FFriendsSendInviteStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
	{}

	virtual ~FFriendsSendInviteStep() = default;

	enum class EState { Init, SendInviteCall, SendInviteCalled, ClearDelegates, Done } State = EState::Init;

	FOnSendInviteComplete SendInvite = FOnSendInviteComplete::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FUniqueNetId& InUserId, const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::SendInviteCalled);
			CHECK(InUserNum == UserNum);
			CHECK(bInWasSuccessful);
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == ListName);
			CHECK(InErrorStr.Len() == 0);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::SendInviteCalled;
			bool Result = OnlineFriendsPtr->SendInvite(UserNum, *UserId->Get(), ListName, SendInvite);

			CHECK(Result == true);
			break;
		}
		case EState::SendInviteCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
