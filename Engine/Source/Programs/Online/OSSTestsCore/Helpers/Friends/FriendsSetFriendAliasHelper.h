// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsSetFriendAliasStep : public FTestPipeline::FStep
{
	FFriendsSetFriendAliasStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId, FString& InLocalListName, FString& InLocalAlias)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
		, ListName(InLocalListName)
		, Alias(InLocalAlias)
	{}

	virtual ~FFriendsSetFriendAliasStep() = default;

	enum class EState { Init, SetFriendAliasCall, SetFriendAliasCalled, ClearDelegates, Done } State = EState::Init;

	FOnSetFriendAliasComplete SetFriendAlias = FOnSetFriendAliasComplete::CreateLambda([this](int32 InUserNum, const FUniqueNetId& InUserId, const FString& InListName, const FOnlineError& InError)
		{
			CHECK(State == EState::SetFriendAliasCalled);
			CHECK(InUserNum == UserNum);
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == ListName);
			CHECK(InError == FOnlineError::Success());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::SetFriendAliasCall;
			break;
		}
		case EState::SetFriendAliasCall:
		{
			State = EState::SetFriendAliasCalled;
			OnlineFriendsPtr->SetFriendAlias(UserNum, *UserId->Get(), ListName, Alias, SetFriendAlias);

			break;
		}
		case EState::SetFriendAliasCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	FString ListName;
	FString Alias; 
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};
