// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsSetFriendSettingsStep : public FTestPipeline::FStep
{
	FFriendsSetFriendSettingsStep(FUniqueNetIdPtr* InLocalUserId, const FString& InLocalSource, bool bInLocalNeverShowAgain)
		: UserId(InLocalUserId)
		, Source(InLocalSource)
		, bNeverShowAgain(bInLocalNeverShowAgain)
	{}

	virtual ~FFriendsSetFriendSettingsStep() = default;

	enum class EState { Init, SetFriendSettingsCall, SetFriendSettingsCalled, ClearDelegates, Done } State = EState::Init;

	FOnSetFriendSettingsComplete SetFriendSettings = FOnSetFriendSettingsComplete::CreateLambda([this](const FUniqueNetId& InUserId, bool bInWasSuccessful, const FString& InErrorStr)
		{
			CHECK(State == EState::SetFriendSettingsCalled);
			CHECK(InUserId == *UserId->Get());
			CHECK(bInWasSuccessful);
			CHECK(InErrorStr.Len() == 0);			
			
			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{

		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::SetFriendSettingsCall;
			break;
		}
		case EState::SetFriendSettingsCall:
		{
			State = EState::SetFriendSettingsCalled;
			OnlineFriendsPtr->SetFriendSettings(*UserId->Get(), Source, bNeverShowAgain, SetFriendSettings);

			break;
		}
		case EState::SetFriendSettingsCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	FString Source;
	bool bNeverShowAgain;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};

