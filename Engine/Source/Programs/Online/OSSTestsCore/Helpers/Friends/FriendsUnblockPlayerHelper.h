// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsUnblockPlayerStep : public FTestPipeline::FStep
{
	FFriendsUnblockPlayerStep(int32 InLocalUserNum, FUniqueNetIdPtr* InLocalUserId)
		: UserNum(InLocalUserNum)
		, UserId(InLocalUserId)
	{}

	virtual ~FFriendsUnblockPlayerStep() = default;

	enum class EState { Init, UnblockPlayerCall, UnblockPlayerCalled, ClearDelegates, Done } State = EState::Init;

	FOnUnblockedPlayerCompleteDelegate UnblockPlayer = FOnUnblockedPlayerCompleteDelegate::CreateLambda([this](int32 InUserNum, bool bInWasSuccessful, const FUniqueNetId& InUserId, const FString& InListName, const FString& InErrorStr)
		{
			CHECK(State == EState::UnblockPlayerCalled);
			CHECK(InUserNum == UserNum);
			CHECK(bInWasSuccessful);
			CHECK(InUserId == *UserId->Get());
			CHECK(InListName == TEXT("BlockedPlayers"));
			CHECK(InErrorStr.Len() == 0);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();

		switch (State)
		{
		case EState::Init:
		{
			OnUnblockPlayerCompleteDelegateHandle = OnlineFriendsPtr->AddOnUnblockedPlayerCompleteDelegate_Handle(UserNum, UnblockPlayer);
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::UnblockPlayerCall;
			break;
		}
		case EState::UnblockPlayerCall:
		{
			State = EState::UnblockPlayerCalled;
			bool Result = OnlineFriendsPtr->UnblockPlayer(UserNum, *UserId->Get());

			CHECK(Result == true);
			break;
		}
		case EState::UnblockPlayerCalled:
		{
			State = EState::UnblockPlayerCalled;
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineFriendsPtr->ClearOnUnblockedPlayerCompleteDelegate_Handle(UserNum, OnUnblockPlayerCompleteDelegateHandle);
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum;
	FUniqueNetIdPtr* UserId = nullptr;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
	FDelegateHandle	OnUnblockPlayerCompleteDelegateHandle;
};
