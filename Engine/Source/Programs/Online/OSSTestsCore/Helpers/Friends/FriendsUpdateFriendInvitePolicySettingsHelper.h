// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineFriendsInterface.h"
#include "OnlineSubsystem.h"

struct FFriendsUpdateFriendInvitePolicySettingsStep : public FTestPipeline::FStep
{
	FFriendsUpdateFriendInvitePolicySettingsStep(FUniqueNetIdPtr* InLocalUserId, EFriendInvitePolicy& InLocalNewInvitesPolicyValue, bool bInAffectsExistingInvites)
		: UserId(InLocalUserId)
		, NewInvitesPolicyValue(InLocalNewInvitesPolicyValue)
	{}

	virtual ~FFriendsUpdateFriendInvitePolicySettingsStep() = default;

	enum class EState { Init, UpdateFriendSettingsCalled, ClearDelegates, Done } State = EState::Init;

	FOnSettingsOperationComplete UpdateFriendInvitePolicySettings = FOnSettingsOperationComplete::CreateLambda([this](const FUniqueNetId& InUserId, bool bInWasSuccessful, bool bInWasUpdate, const FFriendSettings& InSettings, const FString& InErrorStr)
		{
			CHECK(State == EState::UpdateFriendSettingsCalled);
			CHECK(InUserId == *UserId->Get());
			CHECK(bInWasSuccessful);
			CHECK(bInWasUpdate);

			//At the moment we cannot get a correct value for this parameter. Ticket on Jira: OI-3541
			//bool bWasSuccessful = ExpectedSettings.SettingsMap.OrderIndependentCompareEqual(InSettings.SettingsMap);
			//CHECK(bWasSuccessful); //should be true 

			CHECK(InErrorStr.Len() == 0);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{		
		switch (State)
		{
		case EState::Init:
		{
			OnlineFriendsPtr = OnlineSubsystem->GetFriendsInterface();
			REQUIRE(OnlineFriendsPtr != nullptr);

			State = EState::UpdateFriendSettingsCalled;
			OnlineFriendsPtr->UpdateFriendInvitePolicySettings(*UserId->Get(), NewInvitesPolicyValue, true, UpdateFriendInvitePolicySettings);

			break;
		}
		case EState::UpdateFriendSettingsCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	EFriendInvitePolicy NewInvitesPolicyValue;
	IOnlineFriendsPtr OnlineFriendsPtr = nullptr;
};

