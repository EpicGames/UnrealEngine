// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityAutoLoginStep : public FTestPipeline::FStep
{
	FIdentityAutoLoginStep(int32 InLocalUserNum)
		:LocalUserNum(InLocalUserNum)
	{}

	virtual ~FIdentityAutoLoginStep()
	{
		if (OnlineIdentityPtr != nullptr)
		{
			if (OnlineIdentityPtr->OnLoginCompleteDelegates->IsBound())
			{
				OnlineIdentityPtr->OnLoginCompleteDelegates->Clear();
			}
			OnlineIdentityPtr = nullptr;
		}
	};

	enum class EState { Init, AutoLoginCall, AutoLoginCalled, ClearDelegates, Done } State = EState::Init;

	FOnLoginCompleteDelegate LoginDelegate = FOnLoginCompleteDelegate::CreateLambda([this](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
		{
			CHECK(State == EState::AutoLoginCalled);

			CHECK(bLoginWasSuccessful);
			CHECK(LoginLocalUserNum == LocalUserNum);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		switch (State)
		{
			case EState::Init:
			{
				OnLoginCompleteDelegateHandle = OnlineIdentityPtr->AddOnLoginCompleteDelegate_Handle(LocalUserNum, LoginDelegate);

				State = EState::AutoLoginCall;

				break;
			}
			case EState::AutoLoginCall:
			{
				State = EState::AutoLoginCalled;

				bool Result = OnlineIdentityPtr->AutoLogin(LocalUserNum);
				REQUIRE(Result == true);

				break;
			}
			case EState::AutoLoginCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineIdentityPtr->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, OnLoginCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	IOnlineIdentityPtr OnlineIdentityPtr = nullptr;
	FDelegateHandle	OnLoginCompleteDelegateHandle;
};