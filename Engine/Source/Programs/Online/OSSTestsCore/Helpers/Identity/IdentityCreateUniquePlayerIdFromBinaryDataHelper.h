// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityCreateUniquePlayerIdFromBinaryDataStep : public FTestPipeline::FStep
{
	FIdentityCreateUniquePlayerIdFromBinaryDataStep(uint8* InBytes, int32 InSize, TFunction<void(FUniqueNetIdPtr)>&& InStateSaver)
		: Bytes(InBytes)
		, Size(InSize)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityCreateUniquePlayerIdFromBinaryDataStep(uint8* InBytes, int32 InSize)
		: Bytes(InBytes)
		, Size(InSize)
		, StateSaver([](FUniqueNetIdPtr) {})
	{}

	virtual ~FIdentityCreateUniquePlayerIdFromBinaryDataStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		UniqueNetId = OnlineIdentityPtr->CreateUniquePlayerId(Bytes, Size);
		REQUIRE(UniqueNetId.IsValid());

		CHECK(UniqueNetId->ToString().Len() > 0);
	
		StateSaver(UniqueNetId);

		return EContinuance::Done;
	}

protected:
	uint8* Bytes = nullptr;
	int32 Size = 0;
	FUniqueNetIdPtr UniqueNetId = nullptr;
	TFunction<void(FUniqueNetIdPtr)> StateSaver;
};