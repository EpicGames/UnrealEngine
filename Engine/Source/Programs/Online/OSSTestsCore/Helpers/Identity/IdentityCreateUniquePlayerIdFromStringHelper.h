// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityCreateUniquePlayerIdFromStringStep : public FTestPipeline::FStep
{
	FIdentityCreateUniquePlayerIdFromStringStep(const FString& InString, TFunction<void(FUniqueNetIdPtr)>&& InStateSaver)
		: String(InString)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityCreateUniquePlayerIdFromStringStep(const FString& InString)
		: String(InString)
		, StateSaver([](FUniqueNetIdPtr) {})
	{}

	virtual ~FIdentityCreateUniquePlayerIdFromStringStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		REQUIRE(!String.IsEmpty());

		UniqueNetIdPtr = OnlineIdentityPtr->CreateUniquePlayerId(String);
		REQUIRE(UniqueNetIdPtr != nullptr);
		CHECK(UniqueNetIdPtr->ToString() == String);

		StateSaver(UniqueNetIdPtr);

		return EContinuance::Done;
	}

protected:
	FString String = TEXT("");
	FUniqueNetIdPtr UniqueNetIdPtr = nullptr;
	TFunction<void(FUniqueNetIdPtr)> StateSaver;
};