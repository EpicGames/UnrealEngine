// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetAllUserAccountsStep : public FTestPipeline::FStep
{
	FIdentityGetAllUserAccountsStep(TArray<FUniqueNetIdPtr>* InUserUniqueNetIds, TFunction<void(TArray<TSharedPtr<FUserOnlineAccount>>)>&& InStateSaver)
		: UserUniqueNetIds(InUserUniqueNetIds)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetAllUserAccountsStep(TArray<FUniqueNetIdPtr>* InUserUniqueNetIds)
		: UserUniqueNetIds(InUserUniqueNetIds)
		, StateSaver([](TArray<TSharedPtr<FUserOnlineAccount>>){})
	{}

	virtual ~FIdentityGetAllUserAccountsStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		TArray<TSharedPtr<FUserOnlineAccount>> UserAccounts = OnlineIdentityPtr->GetAllUserAccounts();
		REQUIRE(!UserAccounts.IsEmpty());

		for (const FUniqueNetIdPtr UniqueNetId : *UserUniqueNetIds)
		{
			const FString UniqueName = UniqueNetId->ToString();
			const bool bUserAccountExists = UserAccounts.ContainsByPredicate([&UniqueName](const TSharedPtr<FUserOnlineAccount>& UserAccount)
				{
					return UserAccount->GetUserId()->ToString() == UniqueName;
				});
			CHECK(bUserAccountExists);
		}

		StateSaver(UserAccounts);

		return EContinuance::Done;
	}

protected:
	TArray<FUniqueNetIdPtr>* UserUniqueNetIds = nullptr;
	TFunction<void(TArray<TSharedPtr<FUserOnlineAccount>>)> StateSaver;
};