// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetAuthTokenStep : public FTestPipeline::FStep
{
	FIdentityGetAuthTokenStep(int32 InLocalUserNum, TFunction<void(FString)>&& InStateSaver)
		: LocalUserNum(InLocalUserNum)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetAuthTokenStep(int32 InLocalUserNum)
		: LocalUserNum(InLocalUserNum)
		, StateSaver([](FString) {})
	{}

	virtual ~FIdentityGetAuthTokenStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		AuthToken = OnlineIdentityPtr->GetAuthToken(LocalUserNum);
		CHECK(!AuthToken.IsEmpty());
		
		if (OnlineSubsystem->GetSubsystemName() == NULL_SUBSYSTEM) 
		{
			CHECK(AuthToken == TEXT("DummyAuthTicket"));
		}

		StateSaver(AuthToken);

		return EContinuance::Done;
	}

protected:
	int32 LocalUserNum = 0;
	FString AuthToken = TEXT("");
	TFunction<void(FString)> StateSaver;
};