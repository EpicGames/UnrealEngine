// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetAuthTypeStep : public FTestPipeline::FStep
{
	FIdentityGetAuthTypeStep(TFunction<void(FString)>&& InStateSaver)
		: StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetAuthTypeStep()
		: StateSaver([](FString) {})
	{}

	virtual ~FIdentityGetAuthTypeStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		AuthType = OnlineIdentityPtr->GetAuthType();
		if (OnlineSubsystem->GetSubsystemName() == NULL_SUBSYSTEM)
		{
			CHECK(AuthType.IsEmpty());
		}
		else
		{
			CHECK(!AuthType.IsEmpty());
		}

		StateSaver(AuthType);

		return EContinuance::Done;
	}

protected:
	FString AuthType = FString(TEXT(""));
	TFunction<void(FString)> StateSaver;
};