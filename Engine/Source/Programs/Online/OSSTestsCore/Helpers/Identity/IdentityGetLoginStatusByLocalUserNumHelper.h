// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetLoginStatusByLocalUserNumStep : public FTestPipeline::FStep
{
	FIdentityGetLoginStatusByLocalUserNumStep(int32 InLocalUserNum, ELoginStatus::Type InExpectedStatusType)
		: LocalUserNum(InLocalUserNum)
		, ExpectedStatusType(InExpectedStatusType)
	{}

	virtual ~FIdentityGetLoginStatusByLocalUserNumStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		LoginStatusType = OnlineIdentityPtr->GetLoginStatus(LocalUserNum);
		CHECK(LoginStatusType == ExpectedStatusType);

		return EContinuance::Done;
	}

protected:
	int32 LocalUserNum = 0;
	ELoginStatus::Type LoginStatusType = ELoginStatus::Type::NotLoggedIn;
	ELoginStatus::Type ExpectedStatusType = ELoginStatus::Type::NotLoggedIn;
};