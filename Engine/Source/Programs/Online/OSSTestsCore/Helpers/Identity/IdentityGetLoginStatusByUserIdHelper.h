// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetLoginStatusByUserIdStep : public FTestPipeline::FStep
{
	FIdentityGetLoginStatusByUserIdStep(FUniqueNetIdPtr* InUserId, ELoginStatus::Type InExpectedStatus)
		: UserId(InUserId)
		, ExpectedStatus(InExpectedStatus)
	{}

	virtual ~FIdentityGetLoginStatusByUserIdStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		REQUIRE(UserId->IsValid());
		
		LoginStatusType = OnlineIdentityPtr->GetLoginStatus(*UserId->Get());
		CHECK(LoginStatusType == ExpectedStatus);

		return EContinuance::Done;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	ELoginStatus::Type LoginStatusType = ELoginStatus::Type::NotLoggedIn;
	ELoginStatus::Type ExpectedStatus = ELoginStatus::Type::NotLoggedIn;
};