// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetPlatformUserIdFromUniqueNetIdStep : public FTestPipeline::FStep
{
	FIdentityGetPlatformUserIdFromUniqueNetIdStep(FUniqueNetIdPtr* InUserId, TFunction<void(FPlatformUserId)>&& InStateSaver)
		: UserId(InUserId)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetPlatformUserIdFromUniqueNetIdStep(FUniqueNetIdPtr* InUserId)
		: UserId(InUserId)
		, StateSaver([](FPlatformUserId) {})
	{}

	virtual ~FIdentityGetPlatformUserIdFromUniqueNetIdStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		REQUIRE(UserId->IsValid());

		PlatformUserId = OnlineIdentityPtr->GetPlatformUserIdFromUniqueNetId(*UserId->Get());
		CHECK(PlatformUserId != PLATFORMUSERID_NONE);

		StateSaver(PlatformUserId);

		return EContinuance::Done;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	FPlatformUserId PlatformUserId = FPlatformUserId::CreateFromInternalId(0);
	TFunction<void(FPlatformUserId)> StateSaver;
};