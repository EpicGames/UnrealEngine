// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetPlayerNicknameByLocalUserNumStep : public FTestPipeline::FStep
{
	FIdentityGetPlayerNicknameByLocalUserNumStep(int32 InLocalUserNum, TFunction<void(FString)>&& InStateSaver)
		: LocalUserNum(InLocalUserNum)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetPlayerNicknameByLocalUserNumStep(int32 InLocalUserNum)
		: LocalUserNum(InLocalUserNum)
		, StateSaver([](FString) {})
	{}

	virtual ~FIdentityGetPlayerNicknameByLocalUserNumStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		REQUIRE(Nickname.IsEmpty());

		Nickname = OnlineIdentityPtr->GetPlayerNickname(LocalUserNum);
		CHECK(!Nickname.IsEmpty());

		StateSaver(Nickname);

		return EContinuance::Done;
	}

protected:
	int32 LocalUserNum = 0;
	FString Nickname = TEXT("");
	TFunction<void(FString)> StateSaver;
};