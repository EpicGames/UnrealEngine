// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetPlayerNicknameByUserIdStep : public FTestPipeline::FStep
{
	FIdentityGetPlayerNicknameByUserIdStep(FUniqueNetIdPtr* InUserId, TFunction<void(FString)>&& InStateSaver)
		: UserId(InUserId)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetPlayerNicknameByUserIdStep(FUniqueNetIdPtr* InUserId)
		: UserId(InUserId)
		, StateSaver([](FString) {})
	{}

	virtual ~FIdentityGetPlayerNicknameByUserIdStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		REQUIRE(Nickname.IsEmpty());

		Nickname = OnlineIdentityPtr->GetPlayerNickname(*UserId->Get());
		CHECK(!Nickname.IsEmpty());

		StateSaver(Nickname);

		return EContinuance::Done;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	FString Nickname = TEXT("");
	TFunction<void(FString)> StateSaver;
};