// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetUniquePlayerIdStep : public FTestPipeline::FStep
{
	FIdentityGetUniquePlayerIdStep(int32 InLocalUserNum, TFunction<void(FUniqueNetIdPtr)>&& InStateSaver)
		: LocalUserNum(InLocalUserNum)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetUniquePlayerIdStep(int32 InLocalUserNum)
		: LocalUserNum(InLocalUserNum)
		, StateSaver([](FUniqueNetIdPtr) {})
	{}

	virtual ~FIdentityGetUniquePlayerIdStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		UserId = OnlineIdentityPtr->GetUniquePlayerId(LocalUserNum);
		CHECK(UserId != nullptr);

		StateSaver(UserId);

		return EContinuance::Done;
	}

protected:
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr UserId = nullptr;
	TFunction<void(FUniqueNetIdPtr)> StateSaver;
};