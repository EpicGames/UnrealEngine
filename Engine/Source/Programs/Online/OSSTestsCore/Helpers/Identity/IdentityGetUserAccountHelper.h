// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetUserAccountStep : public FTestPipeline::FStep
{
	FIdentityGetUserAccountStep(FUniqueNetIdPtr* InUniqueNetId, TFunction<void(TSharedPtr<FUserOnlineAccount>)>&& InStateSaver)
		: UniqueNetId(InUniqueNetId)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetUserAccountStep(FUniqueNetIdPtr* InUniqueNetId)
		: UniqueNetId(InUniqueNetId)
		, StateSaver([](TSharedPtr<FUserOnlineAccount>) {})
	{}

	virtual ~FIdentityGetUserAccountStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		REQUIRE(UniqueNetId != nullptr);

		UserAccount = OnlineIdentityPtr->GetUserAccount(*UniqueNetId->Get());
		REQUIRE(UserAccount != nullptr);
		CHECK(UserAccount->GetUserId()->ToString() == *UniqueNetId->Get()->ToString());

		StateSaver(UserAccount);

		return EContinuance::Done;
	}

protected:
	TSharedPtr<FUserOnlineAccount> UserAccount = nullptr;
	FUniqueNetIdPtr* UniqueNetId = nullptr;
	TFunction<void(TSharedPtr<FUserOnlineAccount>)> StateSaver;
};