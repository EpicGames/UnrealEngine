// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityGetUserPrivilegeStep : public FTestPipeline::FStep
{
	FIdentityGetUserPrivilegeStep(FUniqueNetIdPtr* InUserId, EUserPrivileges::Type InPrivilege)
		: UserId(InUserId)
		, Privilege(InPrivilege)
	{}

	virtual ~FIdentityGetUserPrivilegeStep() = default;

	enum class EState { CallGetUserPrivilege, Done } State = EState::CallGetUserPrivilege;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::CallGetUserPrivilege:
			{
				IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

				OnlineIdentityPtr->GetUserPrivilege(*UserId->Get(), Privilege, IOnlineIdentity::FOnGetUserPrivilegeCompleteDelegate::CreateLambda([this](const FUniqueNetId& GetUserPrivilegeUniqueId, EUserPrivileges::Type GetUserPrivilegePrivilege, uint32 GetUserPrivilegePrivilegeResult)
					{
						CHECK(GetUserPrivilegeUniqueId == *UserId->Get());
						CHECK(GetUserPrivilegePrivilege == Privilege);
						CHECK(GetUserPrivilegePrivilegeResult == (uint32)IOnlineIdentity::EPrivilegeResults::NoFailures);

						State = EState::Done;
					}));

				break;
			}

			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	EUserPrivileges::Type Privilege = EUserPrivileges::Type::CanUseUserGeneratedContent;
};