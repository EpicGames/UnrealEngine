// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityLoginStep : public FTestPipeline::FStep
{
	FIdentityLoginStep(int32 InLocalUserNum, const FOnlineAccountCredentials& InLocalAccount)
		: LocalUserNum(InLocalUserNum)
		, LocalAccount(InLocalAccount)
	{}

	virtual ~FIdentityLoginStep()
	{
		if (OnlineIdentityPtr != nullptr)
		{
			if (OnlineIdentityPtr->OnLoginCompleteDelegates->IsBound())
			{
				OnlineIdentityPtr->OnLoginCompleteDelegates->Clear();
			}
			OnlineIdentityPtr = nullptr;
		}
	};

	enum class EState { Init, LoginCall, LoginCalled, ClearDelegates, Done } State = EState::Init;

	FOnLoginCompleteDelegate LoginDelegate = FOnLoginCompleteDelegate::CreateLambda([this](int32 LoginLocalUserNum, bool bLoginWasSuccessful, const FUniqueNetId& LoginUserId, const FString& LoginError)
		{
			CHECK(State == EState::LoginCalled);
			CHECK(bLoginWasSuccessful);
			CHECK(LoginLocalUserNum == LocalUserNum);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		switch (State)
		{
			case EState::Init:
			{
				OnLoginCompleteDelegateHandle = OnlineIdentityPtr->AddOnLoginCompleteDelegate_Handle(LocalUserNum, LoginDelegate);

				State = EState::LoginCall;

				break;
			}
			case EState::LoginCall:
			{
				State = EState::LoginCalled;

				bool Result = OnlineIdentityPtr->Login(LocalUserNum, LocalAccount);
				REQUIRE(Result == true);

				break;
			}
			case EState::LoginCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineIdentityPtr->ClearOnLoginCompleteDelegate_Handle(LocalUserNum, OnLoginCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	FOnlineAccountCredentials LocalAccount = {};
	IOnlineIdentityPtr OnlineIdentityPtr = nullptr;
	FDelegateHandle	OnLoginCompleteDelegateHandle;
};