// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

struct FIdentityLogoutStep : public FTestPipeline::FStep
{
	FIdentityLogoutStep(int32 InLocalUserNum)
		: LocalUserNum(InLocalUserNum)
	{}

	virtual ~FIdentityLogoutStep()
	{
		if (OnlineIdentityPtr != nullptr)
		{
			if (OnlineIdentityPtr->OnLogoutCompleteDelegates->IsBound())
			{
				OnlineIdentityPtr->OnLogoutCompleteDelegates->Clear();
			}
			OnlineIdentityPtr = nullptr;
		}
	};

	enum class EState { Init, LogoutCall, LogoutCalled, ClearDelegates, Done } State = EState::Init;

	FOnLogoutCompleteDelegate LogoutDelegate = FOnLogoutCompleteDelegate::CreateLambda([this](int32 LogoutLocalUserNum, bool bLogoutWasSuccessful)
		{
			CHECK(State == EState::LogoutCalled);

			CHECK(bLogoutWasSuccessful);
			CHECK(LogoutLocalUserNum == LocalUserNum);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

		switch (State)
		{
			case EState::Init:
			{
				OnLogoutCompleteDelegateHandle = OnlineIdentityPtr->AddOnLogoutCompleteDelegate_Handle(LocalUserNum, LogoutDelegate);

				State = EState::LogoutCall;

				break;
			}
			case EState::LogoutCall:
			{
				State = EState::LogoutCalled;

				bool Result = OnlineIdentityPtr->Logout(LocalUserNum);
				REQUIRE(Result == true);

				break;
			}
			case EState::LogoutCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineIdentityPtr->ClearOnLogoutCompleteDelegate_Handle(LocalUserNum, OnLogoutCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

	return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	IOnlineIdentityPtr OnlineIdentityPtr = nullptr;
	FDelegateHandle	OnLogoutCompleteDelegateHandle;
};