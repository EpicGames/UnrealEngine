// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineError.h"
#include "OnlineSubsystem.h"

struct FIdentityRevokeAuthTokenStep : public FTestPipeline::FStep
{
	FIdentityRevokeAuthTokenStep(FUniqueNetIdPtr* InLocalUserId)
		: UserId(InLocalUserId)
	{}

	virtual ~FIdentityRevokeAuthTokenStep() = default;

	enum class EState { CallRevokeUserToken, Done } State = EState::CallRevokeUserToken;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::CallRevokeUserToken:
			{
				IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();

				OnlineIdentityPtr->RevokeAuthToken(*UserId->Get(), FOnRevokeAuthTokenCompleteDelegate::CreateLambda([this](const FUniqueNetId& InUserId, const FOnlineError& InError)
					{
						CHECK(*UserId->Get() == InUserId);
						CHECK(InError == FOnlineError::Success());

						State = EState::Done;
					}));

				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}
		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
};