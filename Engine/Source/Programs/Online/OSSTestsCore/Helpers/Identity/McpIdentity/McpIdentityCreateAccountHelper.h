// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineIdentityMcp.h"

struct FMcpIdentityCreateAccountStep : public FTestPipeline::FStep
{
	FMcpIdentityCreateAccountStep(int32 InLocalUserNum, const FCreateAccountInfoMcp& InAccountInfo, TFunction<void(FOnlineAccountCredentials)>&& InStateSaver)
		: LocalUserNum(InLocalUserNum)
		, AccountInfo(InAccountInfo)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FMcpIdentityCreateAccountStep(int32 InLocalUserNum, const FCreateAccountInfoMcp& InAccountInfo)
		: LocalUserNum(InLocalUserNum)
		, AccountInfo(InAccountInfo)
		, StateSaver([](FOnlineAccountCredentials) {})
	{}

	virtual ~FMcpIdentityCreateAccountStep()
	{
		if (OnlineIdentityMcp != nullptr && OnCreateAccountCompleteDelegateHandle.IsValid())
		{
			OnlineIdentityMcp->ClearOnCreateAccountCompleteDelegate_Handle(OnCreateAccountCompleteDelegateHandle);
		}
	};

	enum class EState { CreateAccount, Called, Done } State = EState::CreateAccount;

	FOnCreateAccountCompleteDelegate OnCreateAccountCompleteDelegate = FOnCreateAccountCompleteDelegate::CreateLambda([this](const FUniqueNetId& UserId, const FOnlineAccountCredentials& Credentials, const FOnlineError& Error)
		{
			REQUIRE(State == EState::Called);
			REQUIRE(Error.WasSuccessful());
	
			TestCredentials = Credentials;
			StateSaver(TestCredentials);

			State = EState::Done;
		});
	
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::CreateAccount:
			{
				State = EState::Called;

				OnlineSubsystemMcp = static_cast<FOnlineSubsystemMcp*>(OnlineSubsystem);
				OnlineIdentityMcp = OnlineSubsystemMcp->GetMcpIdentityService();
				REQUIRE(OnlineIdentityMcp.IsValid());

				OnCreateAccountCompleteDelegateHandle = OnlineIdentityMcp->AddOnCreateAccountCompleteDelegate_Handle(OnCreateAccountCompleteDelegate);

				bool Result = OnlineIdentityMcp->CreateAccount(LocalUserNum, AccountInfo);
				CHECK(Result);

				break;
			}

			case EState::Called:
			{
				break;
			}

			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	FOnlineSubsystemMcp* OnlineSubsystemMcp = nullptr;
	FCreateAccountInfoMcp AccountInfo = {};
	FOnlineAccountCredentials TestCredentials = {};
	FOnlineIdentityMcpPtr OnlineIdentityMcp = nullptr;
	FDelegateHandle OnCreateAccountCompleteDelegateHandle;
	TFunction<void(FOnlineAccountCredentials)> StateSaver;
};
