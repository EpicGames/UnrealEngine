// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

template<typename TBaseInnerStep>
struct TInnerStepArrayBuilder
{
	TInnerStepArrayBuilder()
	{}

	template<typename T, typename... TArguments>
	TInnerStepArrayBuilder&& EmplaceInnerStep(TArguments&&... Args)
	{
		Steps.Emplace(new T(std::forward<TArguments>(Args)...));	
		return std::move(*this);
	}

	TArray<TUniquePtr<TBaseInnerStep>> Steps;
};
