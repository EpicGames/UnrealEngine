// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystem.h"

struct FLeaderboardsFlushLeaderboardsStep : public FTestPipeline::FStep
{
	FLeaderboardsFlushLeaderboardsStep(FName& InSessionName)
		: SessionName(InSessionName)
	{}

	virtual ~FLeaderboardsFlushLeaderboardsStep()
	{
		if (OnlineLeaderboardsPtr != nullptr)
		{
			if (OnlineLeaderboardsPtr->OnLeaderboardFlushCompleteDelegates.IsBound())
			{
				OnlineLeaderboardsPtr->OnLeaderboardFlushCompleteDelegates.Clear();
			}
			OnlineLeaderboardsPtr = nullptr;
		}
	}

	enum class EState { Init, FlushLeaderboardsCall, FlushLeaderboardsCalled, ClearDelegates, Done } State = EState::Init;

	FOnLeaderboardFlushCompleteDelegate FlushDelegate = FOnLeaderboardFlushCompleteDelegate::CreateLambda([this](const FName& InSessionName, bool bFlushWasSuccessful)
		{
			CHECK(State == EState::FlushLeaderboardsCalled);
			CHECK(bFlushWasSuccessful);
			CHECK(InSessionName == SessionName);
			
			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineLeaderboardsPtr = OnlineSubsystem->GetLeaderboardsInterface();

		switch (State)
		{
		case EState::Init:
		{
			REQUIRE(OnlineLeaderboardsPtr != nullptr);

			OnFlushCompleteDelegateHandle = OnlineLeaderboardsPtr->AddOnLeaderboardFlushCompleteDelegate_Handle(FlushDelegate);

			State = EState::FlushLeaderboardsCall;
			break;
		}
		case EState::FlushLeaderboardsCall:
		{
			State = EState::FlushLeaderboardsCalled;
			
			bool Result = OnlineLeaderboardsPtr->FlushLeaderboards(SessionName);
			CHECK(Result == true);

			break;
		}
		case EState::FlushLeaderboardsCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineLeaderboardsPtr->ClearOnLeaderboardFlushCompleteDelegate_Handle(OnFlushCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineLeaderboardsPtr OnlineLeaderboardsPtr;
	FName SessionName;
	FUniqueNetIdPtr* PlayerId = nullptr;
	FOnlineLeaderboardWrite WriteObject;
	FDelegateHandle	OnFlushCompleteDelegateHandle;
};
