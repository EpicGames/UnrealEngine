// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystem.h"

struct FLeaderboardsReadLeaderboardsForFriendsStep : public FTestPipeline::FStep
{
	FLeaderboardsReadLeaderboardsForFriendsStep(int32 InLocalUserNum, FOnlineLeaderboardReadRef& InReadObject)
		: ReadObject(InReadObject)
		, LocalUserNum(InLocalUserNum)
	{}

	virtual ~FLeaderboardsReadLeaderboardsForFriendsStep() = default;

	enum class EState { Init, ReadLeaderboardsForFriendsCall, ReadLeaderboardsForFriendsCalled, ClearDelegates, Done } State = EState::Init;

	FOnLeaderboardReadCompleteDelegate ReadDelegate = FOnLeaderboardReadCompleteDelegate::CreateLambda([this](bool bReadWasSuccessful)
		{
			CHECK(State == EState::ReadLeaderboardsForFriendsCalled);
			CHECK(bReadWasSuccessful);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineLeaderboardsPtr = OnlineSubsystem->GetLeaderboardsInterface();
			REQUIRE(OnlineLeaderboardsPtr != nullptr);

			OnReadCompleteDelegateHandle = OnlineLeaderboardsPtr->AddOnLeaderboardReadCompleteDelegate_Handle(ReadDelegate);

			State = EState::ReadLeaderboardsForFriendsCall;
			break;
		}
		case EState::ReadLeaderboardsForFriendsCall:
		{
			State = EState::ReadLeaderboardsForFriendsCalled;

			bool Result = OnlineLeaderboardsPtr->ReadLeaderboardsForFriends(LocalUserNum, ReadObject);
			CHECK(Result == true);

			break;
		}
		case EState::ReadLeaderboardsForFriendsCalled:
		{
 			break;
		}
		case EState::ClearDelegates:
		{
			OnlineLeaderboardsPtr->ClearOnLeaderboardReadCompleteDelegate_Handle(OnReadCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineLeaderboardsPtr OnlineLeaderboardsPtr;
	FOnlineLeaderboardReadRef ReadObject;
	FDelegateHandle	OnReadCompleteDelegateHandle;
	int32 LocalUserNum;
};
