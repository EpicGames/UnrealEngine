// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystem.h"

struct FLeaderboardsReadLeaderboardsStep : public FTestPipeline::FStep
{
	FLeaderboardsReadLeaderboardsStep(TArray<FUniqueNetIdRef>& InPlayers, FOnlineLeaderboardReadRef& InReadObject, TFunction<void(TArray<FUniqueNetIdRef>&)>&& InStateSaver)
		: ReadObject(InReadObject)
		, Players(InPlayers)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FLeaderboardsReadLeaderboardsStep(TArray<FUniqueNetIdRef>& InPlayers, FOnlineLeaderboardReadRef& InReadObject)
		: ReadObject(InReadObject)
		, Players(InPlayers)
		, StateSaver([](TArray<FUniqueNetIdRef>&) {})
	{}

	virtual ~FLeaderboardsReadLeaderboardsStep()
	{
		if (OnlineLeaderboardsPtr != nullptr && OnlineLeaderboardsPtr->OnLeaderboardReadCompleteDelegates.IsBound())
		{
			OnlineLeaderboardsPtr->OnLeaderboardReadCompleteDelegates.Clear();
		}
		OnlineLeaderboardsPtr = nullptr;
	};

	enum class EState { Init, ReadLeaderboardsCall, ReadLeaderboardsCalled, ClearDelegates, Done } State = EState::Init;

	FOnLeaderboardReadCompleteDelegate ReadDelegate = FOnLeaderboardReadCompleteDelegate::CreateLambda([this](bool bReadhWasSuccessful)
	{
		CHECK(State == EState::ReadLeaderboardsCalled);
		CHECK(bReadhWasSuccessful);

		State = EState::ClearDelegates;
	});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			StateSaver(Players);
			OnlineLeaderboardsPtr = OnlineSubsystem->GetLeaderboardsInterface();			
			REQUIRE(OnlineLeaderboardsPtr != nullptr);

			OnReadCompleteDelegateHandle = OnlineLeaderboardsPtr->AddOnLeaderboardReadCompleteDelegate_Handle(ReadDelegate);

			State = EState::ReadLeaderboardsCall;
			break;
		}
		case EState::ReadLeaderboardsCall:
		{
			State = EState::ReadLeaderboardsCalled;

			bool Result = OnlineLeaderboardsPtr->ReadLeaderboards(Players, ReadObject);
			CHECK(Result == true);
	
			break;
		}
		case EState::ReadLeaderboardsCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineLeaderboardsPtr->ClearOnLeaderboardReadCompleteDelegate_Handle(OnReadCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineLeaderboardsPtr OnlineLeaderboardsPtr;
	FOnlineLeaderboardReadRef ReadObject;
	TArray<FUniqueNetIdRef> Players;
	TFunction<void(TArray<FUniqueNetIdRef>&)> StateSaver;
	FDelegateHandle	OnReadCompleteDelegateHandle;
};
