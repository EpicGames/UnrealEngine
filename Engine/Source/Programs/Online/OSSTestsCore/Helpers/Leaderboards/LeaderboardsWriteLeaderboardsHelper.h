// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineLeaderboardInterface.h"
#include "OnlineSubsystem.h"

struct FLeaderboardsWriteLeaderboardsStep : public FTestPipeline::FStep
{
	FLeaderboardsWriteLeaderboardsStep(FName& InSessionName, FUniqueNetIdPtr* InPlayerId, FOnlineLeaderboardWrite& InWriteObject)
		: SessionName(InSessionName)
		, PlayerId(InPlayerId)
		, WriteObject(InWriteObject)
	{}

	virtual ~FLeaderboardsWriteLeaderboardsStep() = default;

	enum class EState { Init, WriteLeaderboardsCall, WriteLeaderboardsCalled, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineLeaderboardsPtr = OnlineSubsystem->GetLeaderboardsInterface();
			REQUIRE(OnlineLeaderboardsPtr != nullptr);
			
			State = EState::WriteLeaderboardsCall;
			break;
		}
		case EState::WriteLeaderboardsCall:
		{
			bool Result = OnlineLeaderboardsPtr->WriteLeaderboards(SessionName, *PlayerId->Get(), WriteObject);
			CHECK(Result == true);

			State = EState::WriteLeaderboardsCalled;
			break;
		}
		case EState::WriteLeaderboardsCalled:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	IOnlineLeaderboardsPtr OnlineLeaderboardsPtr;
	FName SessionName;
	FUniqueNetIdPtr* PlayerId = nullptr;
	FOnlineLeaderboardWrite WriteObject;
};
