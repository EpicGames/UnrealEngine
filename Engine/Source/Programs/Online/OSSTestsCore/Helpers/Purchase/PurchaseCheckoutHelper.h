// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineError.h"
#include "OnlineSubsystem.h"

struct FPurchaseCheckoutStep : public FTestPipeline::FStep
{
	FPurchaseCheckoutStep(FUniqueNetIdPtr* InUserId, FPurchaseCheckoutRequest* InCheckoutRequest, const FString& InUniqueId, const FString& InItemName)
		: UserId(InUserId)
		, CheckoutRequest(InCheckoutRequest)
		, UniqueId(InUniqueId)
		, ItemName(InItemName)
	{}

	virtual ~FPurchaseCheckoutStep() = default;

	enum class EState { Init, CheckoutCall, CheckoutCalled, FinalizePurchaseCall, FinalizePurchaseCalled, Done } State = EState::Init;

	FOnPurchaseCheckoutComplete CheckoutDelegate = FOnPurchaseCheckoutComplete::CreateLambda([this](const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
		{
			CHECK(State == EState::CheckoutCall);
			CHECK(Result.bSucceeded == true);

			CHECK(Receipt->ReceiptOffers[0].LineItems[0].UniqueId == UniqueId);
			CHECK(Receipt->ReceiptOffers[0].LineItems[0].ItemName == ItemName);
			
			State = EState::FinalizePurchaseCall;
		});
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlinePurchasePtr = OnlineSubsystem->GetPurchaseInterface();
			REQUIRE(OnlinePurchasePtr != nullptr);

			State = EState::CheckoutCall;
			break;
		}
		case EState::CheckoutCall:
		{
			OnlinePurchasePtr->Checkout(*UserId->Get(), *CheckoutRequest, CheckoutDelegate);
			break;
		}
		case EState::CheckoutCalled:
		{
			State = EState::Done;
			break;
		}
		case EState::FinalizePurchaseCall:
		{
			OnlinePurchasePtr->FinalizePurchase(*UserId->Get(), UniqueId);
			State = EState::FinalizePurchaseCalled;
			break;
		}
		case EState::FinalizePurchaseCalled:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	IOnlinePurchasePtr OnlinePurchasePtr = nullptr;
	FPurchaseCheckoutRequest* CheckoutRequest = nullptr;
	FString UniqueId;
	FString ItemName;
};