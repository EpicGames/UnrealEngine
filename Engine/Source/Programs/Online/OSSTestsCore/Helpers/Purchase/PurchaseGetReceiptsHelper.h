// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineError.h"
#include "OnlineSubsystem.h"

struct FPurchaseGetReceiptsStep : public FTestPipeline::FStep
{
	FPurchaseGetReceiptsStep(FUniqueNetIdPtr* InUserId, TFunction<void(TArray<FPurchaseReceipt>*)>&& InStateSaver, const FString& InUniqueId = FString(""))
		: UserId(InUserId)
		, StateSaver(MoveTemp(InStateSaver))
		, UniqueId(InUniqueId)
	{}


	FPurchaseGetReceiptsStep(FUniqueNetIdPtr* InUserId)
		: UserId(InUserId)
		, StateSaver([](TArray<FPurchaseReceipt>*) {})
	{

	}

	virtual ~FPurchaseGetReceiptsStep() = default;
	
	enum class EState { Init, GetReceiptsCall, GetReceiptsCalled, FinalizePurchaseCall, FinalizePurchaseCalled, Done } State = EState::Init;
	
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlinePurchasePtr = OnlineSubsystem->GetPurchaseInterface();
			REQUIRE(OnlinePurchasePtr != nullptr);

			State = EState::GetReceiptsCall;
			break;
		}
		case EState::GetReceiptsCall:
		{
			OnlinePurchasePtr->GetReceipts(*UserId->Get(), OutReceipt);
		
			StateSaver(&OutReceipt);

			State = EState::GetReceiptsCalled;
			break;
		}
		case EState::GetReceiptsCalled:
		{
			State = EState::FinalizePurchaseCall;
			break;
		}
		case EState::FinalizePurchaseCall:
		{
			OnlinePurchasePtr->FinalizePurchase(*UserId->Get(), UniqueId);
			
			State = EState::FinalizePurchaseCalled;
			break;
		}
		case EState::FinalizePurchaseCalled:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	TArray<FPurchaseReceipt> OutReceipt = {};
	IOnlinePurchasePtr OnlinePurchasePtr = nullptr;
	TFunction<void(TArray<FPurchaseReceipt>*)> StateSaver;
	FString UniqueId;
};