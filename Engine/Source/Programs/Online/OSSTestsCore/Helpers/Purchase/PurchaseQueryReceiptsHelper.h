// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineError.h"
#include "OnlineSubsystem.h"

struct FPurchaseQueryReceiptsStep : public FTestPipeline::FStep
{
	FPurchaseQueryReceiptsStep(FUniqueNetIdPtr* InUserId, const bool& bInRestoreReceipts, const FString& InReceiptId)
		: UserId(InUserId)
		, bRestoreReceipts(bInRestoreReceipts)
		, ReceiptId(InReceiptId)
	{}

	FPurchaseQueryReceiptsStep(FUniqueNetIdPtr* InUserId, const bool& bInRestoreReceipts)
		: UserId(InUserId)
		, bRestoreReceipts(bInRestoreReceipts)
	{
	}
	
	virtual ~FPurchaseQueryReceiptsStep() = default;
	
	enum class EState { Init, QueryReceiptsCall, QueryReceiptsCalled, FinalizePurchaseCall, FinalizePurchaseCalled, Done } State = EState::Init;
	
	FOnQueryReceiptsComplete QueryReceipts = FOnQueryReceiptsComplete::CreateLambda([this](const FOnlineError& Result)
		{
			CHECK(State == EState::QueryReceiptsCalled);
			CHECK(Result.bSucceeded == true);

			State = EState::FinalizePurchaseCall;
		});
	
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlinePurchasePtr = OnlineSubsystem->GetPurchaseInterface();
				REQUIRE(OnlinePurchasePtr != nullptr);

				State = EState::QueryReceiptsCall;
				break;
			}
			case EState::QueryReceiptsCall:
			{
				OnlinePurchasePtr->QueryReceipts(*UserId->Get(), bRestoreReceipts, QueryReceipts);

				State = EState::QueryReceiptsCalled;
				break;
			}
			case EState::QueryReceiptsCalled:
			{
				break;
			}
			case EState::FinalizePurchaseCall:
			{
				OnlinePurchasePtr->FinalizePurchase(*UserId->Get(), ReceiptId);
				
				State = EState::FinalizePurchaseCalled;
				break;
			}
			case EState::FinalizePurchaseCalled:
			{
				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	IOnlinePurchasePtr OnlinePurchasePtr = nullptr;
	bool bRestoreReceipts = false;
	FString ReceiptId;
};
