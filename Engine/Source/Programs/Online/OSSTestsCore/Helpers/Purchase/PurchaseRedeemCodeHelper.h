// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlinePurchaseInterface.h"
#include "OnlineSubsystem.h"

struct FPurchaseRedeemCodeStep : public FTestPipeline::FStep
{
	FPurchaseRedeemCodeStep(FUniqueNetIdPtr* InUserId, const FRedeemCodeRequest& InRedeemCodeRequest, const FString& InUniqueId, const FString& InItemName)
		: UserId(InUserId)
		, RedeemCodeRequest(InRedeemCodeRequest)
		, UniqueId(InUniqueId)
		, ItemName(InItemName)
	{}

	virtual ~FPurchaseRedeemCodeStep() = default;
	
	FOnPurchaseRedeemCodeComplete PurchaseRedeemCode = FOnPurchaseRedeemCodeComplete::CreateLambda([this](const FOnlineError& Result, const TSharedRef<FPurchaseReceipt>& Receipt)
		{
			CHECK(State == EState::RedeemCodeCall);
			CHECK(Result.bSucceeded == true);

			CHECK(Receipt->ReceiptOffers[0].LineItems[0].UniqueId == UniqueId);
			CHECK(Receipt->ReceiptOffers[0].LineItems[0].ItemName == ItemName);

			State = EState::RedeemCodeCalled;
		});

	enum class EState { Init, RedeemCodeCall,RedeemCodeCalled, Done } State = EState::Init;
	
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlinePurchasePtr = OnlineSubsystem->GetPurchaseInterface();
			REQUIRE(OnlinePurchasePtr != nullptr);

			State = EState::RedeemCodeCall;
			break;
		}
		case EState::RedeemCodeCall:
		{
			OnlinePurchasePtr->RedeemCode(*UserId->Get(), RedeemCodeRequest, PurchaseRedeemCode);
		
			break;
		}
		case EState::RedeemCodeCalled:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FUniqueNetIdPtr* UserId = nullptr;
	IOnlinePurchasePtr OnlinePurchasePtr = nullptr;
	FRedeemCodeRequest RedeemCodeRequest;
	FString UniqueId;
	FString ItemName;
};