// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionCancelFindSessionStep : public FTestPipeline::FStep
{
	FSessionCancelFindSessionStep(FUniqueNetIdPtr* InSearchingUserId, FUniqueNetIdPtr* InFriendId, const TSharedRef<FOnlineSessionSearch>& InSearchSettings)
		: SearchingUserId(InSearchingUserId)
		, FriendId(InFriendId)
		, SearchSettings(InSearchSettings)
	{}

	virtual ~FSessionCancelFindSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnCancelFindSessionsCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnCancelFindSessionsCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, CancelFindSessionCall, CancelFindSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnFindSessionsCompleteDelegate FindSessionsDelegate = FOnFindSessionsCompleteDelegate::CreateLambda([this](bool bWasSuccessful){});

	FOnCancelFindSessionsCompleteDelegate CancelFindSessionsDelegate = FOnCancelFindSessionsCompleteDelegate::CreateLambda([this](bool bWasSuccessful)
		{
			REQUIRE(State == EState::CancelFindSessionCalled);
			CHECK(bWasSuccessful);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			OnFindSessionsCompleteDelegateHandle = OnlineSessionPtr->AddOnFindSessionsCompleteDelegate_Handle(FindSessionsDelegate);
			OnCancelFindSessionsCompleteDelegateHandle = OnlineSessionPtr->AddOnCancelFindSessionsCompleteDelegate_Handle(CancelFindSessionsDelegate);
			State = EState::CancelFindSessionCall;

			break;
		}
		case EState::CancelFindSessionCall:
		{
			State = EState::CancelFindSessionCalled;
			bool Result = OnlineSessionPtr->FindSessions(*SearchingUserId->Get(), SearchSettings);
			REQUIRE(Result == true);
			Result = false;

			Result = OnlineSessionPtr->CancelFindSessions();
			REQUIRE(Result == true);

			break;
		}
		case EState::CancelFindSessionCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnCancelFindSessionsCompleteDelegate_Handle(OnCancelFindSessionsCompleteDelegateHandle);
			OnlineSessionPtr->ClearOnFindSessionsCompleteDelegate_Handle(OnFindSessionsCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr* SearchingUserId = nullptr;
	FUniqueNetIdPtr* FriendId = nullptr;
	TSharedPtr<FNamedOnlineSession>* NamedOnlineSession = nullptr;
	FString SessionKey = TEXT("");
	TSharedRef<FOnlineSessionSearch> SearchSettings;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
	FDelegateHandle	OnFindSessionsCompleteDelegateHandle;
	FDelegateHandle	OnCancelFindSessionsCompleteDelegateHandle;
};