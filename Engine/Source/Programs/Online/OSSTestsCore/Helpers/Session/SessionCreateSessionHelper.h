// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionCreateSessionStep : public FTestPipeline::FStep
{
	FSessionCreateSessionStep(FUniqueNetIdPtr* InLocalUserId, FName InSessionName, const FOnlineSessionSettings& InNewSessionSettings, TFunction<void(TSharedPtr<FNamedOnlineSession>)>&& InStateSaver = [](TSharedPtr<FNamedOnlineSession>){})
		: LocalUserId(InLocalUserId)
		, TestSessionName(InSessionName)
		, NewSessionSettings(InNewSessionSettings)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FSessionCreateSessionStep(int32 InLocalUserNum, FName InSessionName, const FOnlineSessionSettings& InNewSessionSettings, TFunction<void(TSharedPtr<FNamedOnlineSession>)>&& InStateSaver = [](TSharedPtr<FNamedOnlineSession>) {})
		: LocalUserNum(InLocalUserNum)
		, TestSessionName(InSessionName)
		, NewSessionSettings(InNewSessionSettings)
		, StateSaver(MoveTemp(InStateSaver))
		, bUseOverload(true)
	{}

	virtual ~FSessionCreateSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnCreateSessionCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnCreateSessionCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, CreateSessionCall, CreateSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnCreateSessionCompleteDelegate CreateSessionDelegate = FOnCreateSessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bWasSuccessful)
	{
		REQUIRE(State == EState::CreateSessionCalled);
		CHECK(bWasSuccessful);
		CHECK(SessionName.ToString() == TestSessionName.ToString());

		TSharedPtr<FNamedOnlineSession> NamedOnlineSession = MakeShared<FNamedOnlineSession>(*OnlineSessionPtr->GetNamedSession(TestSessionName));
		StateSaver(MoveTemp(NamedOnlineSession));

		State = EState::ClearDelegates;
	});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
				REQUIRE(OnlineSessionPtr != nullptr);

				OnCreateSessionCompleteDelegateHandle = OnlineSessionPtr->AddOnCreateSessionCompleteDelegate_Handle(CreateSessionDelegate);
				State = EState::CreateSessionCall;

				break;
			}
			case EState::CreateSessionCall:
			{
				State = EState::CreateSessionCalled;

				bool Result = false; 
				if (bUseOverload)
				{
					Result = OnlineSessionPtr->CreateSession(LocalUserNum, TestSessionName, NewSessionSettings);
				}
				else
				{
					Result = OnlineSessionPtr->CreateSession(*LocalUserId->Get(), TestSessionName, NewSessionSettings);
				}
				REQUIRE(Result == true);
		
				break;
			}
			case EState::CreateSessionCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineSessionPtr->ClearOnCreateSessionCompleteDelegate_Handle(OnCreateSessionCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr* LocalUserId = nullptr;
	FName TestSessionName = {};
	FOnlineSessionSettings NewSessionSettings = {};
	TFunction<void(TSharedPtr<FNamedOnlineSession>)> StateSaver;
	bool bUseOverload = false;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
	FDelegateHandle	OnCreateSessionCompleteDelegateHandle;
	
};