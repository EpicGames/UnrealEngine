// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionDestroySessionStep : public FTestPipeline::FStep
{
	FSessionDestroySessionStep(FName InSessionName)
		: TestSessionName(InSessionName)
	{}

	virtual ~FSessionDestroySessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnDestroySessionCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnDestroySessionCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { DestroySessionCall, DestroySessionCalled, ClearDelegates, Done } State = EState::DestroySessionCall;

	FOnDestroySessionCompleteDelegate DestroySessionDelegate = FOnDestroySessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bWasSuccessful)
		{
			REQUIRE(State == EState::DestroySessionCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::DestroySessionCall:
			{
				OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
				REQUIRE(OnlineSessionPtr != nullptr);

				State = EState::DestroySessionCalled;

				bool Result = OnlineSessionPtr->DestroySession(TestSessionName, DestroySessionDelegate);
				REQUIRE(Result == true);

				break;
			}
			case EState::DestroySessionCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineSessionPtr->ClearOnDestroySessionCompleteDelegate_Handle(OnDestroySessionCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
	FDelegateHandle	OnDestroySessionCompleteDelegateHandle;
};