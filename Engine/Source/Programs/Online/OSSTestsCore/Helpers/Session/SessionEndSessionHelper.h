// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionEndSessionStep : public FTestPipeline::FStep
{
	FSessionEndSessionStep(FName InSessionName)
		: TestSessionName(InSessionName)
	{}

	virtual ~FSessionEndSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnEndSessionCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnEndSessionCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, EndSessionCall, EndSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnEndSessionCompleteDelegate EndSessionDelegate = FOnEndSessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bWasSuccessful)
		{
			REQUIRE(State == EState::EndSessionCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			OnEndSessionCompleteDelegateHandle = OnlineSessionPtr->AddOnEndSessionCompleteDelegate_Handle(EndSessionDelegate);

			State = EState::EndSessionCall;

			break;
		}
		case EState::EndSessionCall:
		{
			State = EState::EndSessionCalled;

			bool Result = OnlineSessionPtr->EndSession(TestSessionName);
			REQUIRE(Result == true);

			break;
		}
		case EState::EndSessionCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnEndSessionCompleteDelegate_Handle(OnEndSessionCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
	FDelegateHandle	OnEndSessionCompleteDelegateHandle;
};