// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionFindFriendSessionStep : public FTestPipeline::FStep
{
	FSessionFindFriendSessionStep(FUniqueNetIdPtr* InUserId, int32 InUserNum, FUniqueNetIdPtr* InFriendId)
		: UserNum(InUserNum)
		, UserId(InUserId)
		, FriendId(InFriendId)
	{}

	virtual ~FSessionFindFriendSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnFindFriendSessionCompleteDelegates->IsBound())
			{
				OnlineSessionPtr->OnFindFriendSessionCompleteDelegates->Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, FindFriendSessionCall, FindFriendSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnFindFriendSessionCompleteDelegate FindFriendSessionDelegate = FOnFindFriendSessionCompleteDelegate::CreateLambda([this](int32 InLocalUserNum, bool bWasSuccessful, const TArray<FOnlineSessionSearchResult>& InSearchResult)
		{
			REQUIRE(State == EState::FindFriendSessionCalled);
			CHECK(UserNum == InLocalUserNum);
			CHECK(bWasSuccessful);
			CHECK(InSearchResult.Num() > 0);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			OnFindFriendSessionCompleteDelegateHandle = OnlineSessionPtr->AddOnFindFriendSessionCompleteDelegate_Handle(UserNum, FindFriendSessionDelegate);
			State = EState::FindFriendSessionCall;
			break;
		}
		case EState::FindFriendSessionCall:
		{
			State = EState::FindFriendSessionCalled;

			bool Result = OnlineSessionPtr->FindFriendSession(*UserId->Get(), *FriendId->Get());
			REQUIRE(Result == true);
			break;
		}
		case EState::FindFriendSessionCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnCreateSessionCompleteDelegate_Handle(OnFindFriendSessionCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 UserNum = 0;
	FUniqueNetIdPtr* UserId = nullptr;
	FUniqueNetIdPtr* FriendId = nullptr;
	bool bUseOverload = false;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
	FDelegateHandle OnFindFriendSessionCompleteDelegateHandle;
};