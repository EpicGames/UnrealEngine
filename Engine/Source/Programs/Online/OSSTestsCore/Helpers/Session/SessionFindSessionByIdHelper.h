// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionFindSessionByIdStep : public FTestPipeline::FStep
{
	FSessionFindSessionByIdStep(FUniqueNetIdPtr* InSearchingUserId, FUniqueNetIdPtr* InFriendId, TSharedPtr<FNamedOnlineSession>* InNamedOnlineSession, TFunction<void(TSharedPtr<FOnlineSessionSearchResult>)>&& InStateSaver = [](TSharedPtr<FOnlineSessionSearchResult>) {})
		: SearchingUserId(InSearchingUserId)
		, FriendId(InFriendId)
		, NamedOnlineSession(InNamedOnlineSession)
		, StateSaver(InStateSaver)
	{}

	FSessionFindSessionByIdStep(FUniqueNetIdPtr* InSearchingUserId, FUniqueNetIdPtr* InFriendId, TSharedPtr<FNamedOnlineSession>* InNamedOnlineSession, const FString& InSessionKey, TFunction<void(TSharedPtr<FOnlineSessionSearchResult>)>&& InStateSaver = [](TSharedPtr<FOnlineSessionSearchResult>) {})
		: SearchingUserId(InSearchingUserId)
		, FriendId(InFriendId)
		, NamedOnlineSession(InNamedOnlineSession)
		, SessionKey(InSessionKey)
		, StateSaver(InStateSaver)
		, bUseOverload(true)
	{}

	virtual ~FSessionFindSessionByIdStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnFindSessionsCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnFindSessionsCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, FindSessionByIdCall, FindSessionByIdCalled, ClearDelegates, Done } State = EState::Init;

	FOnSingleSessionResultCompleteDelegate FindSessionsByIdDelegate = FOnSingleSessionResultCompleteDelegate::CreateLambda([this](int32 InLocalUserNum, bool bWasSuccessful, const FOnlineSessionSearchResult& InSearchResult)
		{
			REQUIRE(State == EState::FindSessionByIdCalled);
			CHECK(LocalUserNum == InLocalUserNum);
			CHECK(bWasSuccessful);
			REQUIRE(NamedOnlineSession != nullptr);
			REQUIRE(NamedOnlineSession->IsValid() == true);
			REQUIRE(NamedOnlineSession->ToSharedRef()->SessionInfo.IsValid() == true);
			REQUIRE(InSearchResult.Session.SessionInfo.IsValid() == true);
			CHECK(NamedOnlineSession->ToSharedRef()->SessionInfo->GetSessionId() == InSearchResult.Session.SessionInfo->GetSessionId());

			TSharedPtr<FOnlineSessionSearchResult> SearchResultPtr = MakeShared<FOnlineSessionSearchResult>(InSearchResult);
			StateSaver(MoveTemp(SearchResultPtr));

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			State = EState::FindSessionByIdCall;
			break;
		}
		case EState::FindSessionByIdCall:
		{
			State = EState::FindSessionByIdCalled;

			bool Result = false;
			if (bUseOverload)
			{
				Result = OnlineSessionPtr->FindSessionById(*SearchingUserId->Get(), NamedOnlineSession->ToSharedRef()->SessionInfo->GetSessionId(), *FriendId->Get(), SessionKey, FindSessionsByIdDelegate);
			}
			else
			{
				Result = OnlineSessionPtr->FindSessionById(*SearchingUserId->Get(), NamedOnlineSession->ToSharedRef()->SessionInfo->GetSessionId(), *FriendId->Get(), FindSessionsByIdDelegate);
			}
			REQUIRE(Result == true);
			break;
		}
		case EState::FindSessionByIdCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr* SearchingUserId = nullptr;
	FUniqueNetIdPtr* FriendId = nullptr;
	TSharedPtr<FNamedOnlineSession>* NamedOnlineSession = nullptr;
	FString SessionKey = TEXT("");
	TFunction<void(TSharedPtr<FOnlineSessionSearchResult>)> StateSaver;
	bool bUseOverload = false;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};