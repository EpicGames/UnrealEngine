// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionFindSessionStep : public FTestPipeline::FStep
{
	FSessionFindSessionStep(int32 InLocalUserNum, const TSharedRef<FOnlineSessionSearch>& InSearchSettings)
		: LocalUserNum(InLocalUserNum)
		, SearchSettings(InSearchSettings)
	{}
	FSessionFindSessionStep(FUniqueNetIdPtr* InSearchingPlayerId, const TSharedRef<FOnlineSessionSearch>& InSearchSettings)
		: SearchingPlayerId(InSearchingPlayerId)
		, SearchSettings(InSearchSettings)
		, bUseOverload(true)
	{}
	virtual ~FSessionFindSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnFindSessionsCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnFindSessionsCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};
	enum class EState { Init, FindSessionsCall, FindSessionsCalled, ClearDelegates, Done } State = EState::Init;
	FOnFindSessionsCompleteDelegate FindSessionsDelegate = FOnFindSessionsCompleteDelegate::CreateLambda([this](bool bWasSuccessful)
		{
			REQUIRE(State == EState::FindSessionsCalled);
			REQUIRE(SearchSettings->SearchState == EOnlineAsyncTaskState::Done);
			CHECK(bWasSuccessful);
			State = EState::ClearDelegates;
		});
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);
			OnFindSessionsCompleteDelegateHandle = OnlineSessionPtr->AddOnFindSessionsCompleteDelegate_Handle(FindSessionsDelegate);
			State = EState::FindSessionsCall;
			break;
		}
		case EState::FindSessionsCall:
		{
			State = EState::FindSessionsCalled;
			bool Result = false;
			if (bUseOverload)
			{
				Result = OnlineSessionPtr->FindSessions(*SearchingPlayerId->Get(), SearchSettings);
			}
			else
			{
				Result = OnlineSessionPtr->FindSessions(LocalUserNum, SearchSettings);
			}
			REQUIRE(Result == true);
			break;
		}
		case EState::FindSessionsCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnFindSessionsCompleteDelegate_Handle(OnFindSessionsCompleteDelegateHandle);
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}
		return EContinuance::ContinueStepping;
	}
protected:
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr* SearchingPlayerId = nullptr;
	TSharedRef<FOnlineSessionSearch> SearchSettings;
	bool bUseOverload = false;
	FDelegateHandle	OnFindSessionsCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};