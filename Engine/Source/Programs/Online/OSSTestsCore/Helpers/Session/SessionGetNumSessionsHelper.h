// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionGetNumSessionsStep : public FTestPipeline::FStep
{
	FSessionGetNumSessionsStep(int32 InExpectedSessionsNum)
		: ExpectedSessionsNum(InExpectedSessionsNum)
	{}

	virtual ~FSessionGetNumSessionsStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
		REQUIRE(OnlineSessionPtr != nullptr);

		int32 SessionsNum = OnlineSessionPtr->GetNumSessions();
		CHECK(SessionsNum == ExpectedSessionsNum);

		return EContinuance::Done;
	}

protected:
	int32 ExpectedSessionsNum = 0;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};