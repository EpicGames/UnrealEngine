// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionGetResolvedConnectStringStep : public FTestPipeline::FStep
{
	FSessionGetResolvedConnectStringStep(FName InSessionName, FString& InConnectInfo, FName InPortType)
		: PortType(InPortType)
		, ConnectInfo(InConnectInfo)
		, SessionName(InSessionName)
	{}

	FSessionGetResolvedConnectStringStep(TSharedPtr<FOnlineSessionSearchResult>* InSearchResult, FName InPortType, FString& InConnectInfo)
		: SearchResult(InSearchResult)
		, PortType(InPortType)
		, ConnectInfo(InConnectInfo)
		, bUseOverload(true)
	{}

	virtual ~FSessionGetResolvedConnectStringStep() = default;

	enum class EState { Init, GetResolvedConnectStringCall, GetResolvedConnectStringCalled, ClearDelegates, Done } State = EState::Init;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			State = EState::GetResolvedConnectStringCall;
			break;
		}
		case EState::GetResolvedConnectStringCall:
		{
			State = EState::GetResolvedConnectStringCalled;

			bool Result = false;
			if (bUseOverload)
			{
				Result = OnlineSessionPtr->GetResolvedConnectString(*SearchResult->Get(), PortType, ConnectInfo);
			}
			else
			{
				Result = OnlineSessionPtr->GetResolvedConnectString(SessionName, ConnectInfo, PortType);
			}
			REQUIRE(Result == true);
			REQUIRE(ConnectInfo.Len() > 0);
			ConnectInfo.Empty();
			break;
		}
		case EState::GetResolvedConnectStringCalled:
		{
			State = EState::ClearDelegates;
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	TSharedPtr<FOnlineSessionSearchResult>* SearchResult = nullptr;
	FName PortType = TEXT("");
	FString ConnectInfo = TEXT("");
	FName SessionName = TEXT("");
	bool bUseOverload = false;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};