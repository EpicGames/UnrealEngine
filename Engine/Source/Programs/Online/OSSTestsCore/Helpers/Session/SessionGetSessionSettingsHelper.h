// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionGetSessionSettingsStep : public FTestPipeline::FStep
{
	FSessionGetSessionSettingsStep(FName InSessionName, const FOnlineSessionSettings& InExpectedSessionSettings, TFunction<void(FOnlineSessionSettings)>&& InStateSaver)
		: SessionName(InSessionName)
		, ExpectedSessionSettings(InExpectedSessionSettings)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FSessionGetSessionSettingsStep(FName InSessionName, const FOnlineSessionSettings& InExpectedSessionSettings)
		: SessionName(InSessionName)
		, ExpectedSessionSettings(InExpectedSessionSettings)
		, StateSaver([](FOnlineSessionSettings) {})
	{}

	virtual ~FSessionGetSessionSettingsStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
		REQUIRE(OnlineSessionPtr != nullptr);

		FOnlineSessionSettings* SessionSettings = OnlineSessionPtr->GetSessionSettings(SessionName);
		REQUIRE(SessionSettings != nullptr);

		CHECK(SessionSettings->bAllowInvites == ExpectedSessionSettings.bAllowInvites);
		CHECK(SessionSettings->bAllowJoinInProgress == ExpectedSessionSettings.bAllowJoinInProgress);
		CHECK(SessionSettings->bAllowJoinViaPresence == ExpectedSessionSettings.bAllowJoinViaPresence);
		CHECK(SessionSettings->bAntiCheatProtected == ExpectedSessionSettings.bAntiCheatProtected);

		StateSaver(*SessionSettings);

		return EContinuance::Done;
	}

protected:
	FName SessionName = {};
	FOnlineSessionSettings ExpectedSessionSettings = {};
	TFunction<void(FOnlineSessionSettings)> StateSaver;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};