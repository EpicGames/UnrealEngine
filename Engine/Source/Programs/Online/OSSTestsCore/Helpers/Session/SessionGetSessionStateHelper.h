// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionGetSessionStateStep : public FTestPipeline::FStep
{
	FSessionGetSessionStateStep(FName InSessionName, const EOnlineSessionState::Type InExpectedSessionState)
		: SessionName(InSessionName)
		, ExpectedSessionState(InExpectedSessionState)
	{}

	virtual ~FSessionGetSessionStateStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
		REQUIRE(OnlineSessionPtr != nullptr);

		EOnlineSessionState::Type SessionState = OnlineSessionPtr->GetSessionState(SessionName);
		CHECK(SessionState == ExpectedSessionState);

		return EContinuance::Done;
	}

protected:
	FName SessionName = {};
	EOnlineSessionState::Type ExpectedSessionState = EOnlineSessionState::Type::NoSession;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};