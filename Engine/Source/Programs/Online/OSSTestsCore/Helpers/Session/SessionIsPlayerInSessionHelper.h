// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionIsPlayerInSessionStep : public FTestPipeline::FStep
{
	FSessionIsPlayerInSessionStep(FName InSessionName, FUniqueNetIdPtr* InPlayer)
		: TestSessionName(InSessionName)
		, Player(InPlayer)
	{}

	virtual ~FSessionIsPlayerInSessionStep() = default;

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
		REQUIRE(OnlineSessionPtr != nullptr);

		bIsPlayerInSession = OnlineSessionPtr->IsPlayerInSession(TestSessionName, *Player->Get());
		CHECK(bIsPlayerInSession);

		return EContinuance::Done;
	}

protected:
	FName TestSessionName = {};
	FUniqueNetIdPtr* Player = nullptr;
	bool bIsPlayerInSession = false;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};