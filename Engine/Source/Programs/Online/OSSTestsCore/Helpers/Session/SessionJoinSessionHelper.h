// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionJoinSessionStep : public FTestPipeline::FStep
{
	FSessionJoinSessionStep(int32 InPlayerNum, const FName& InSessionName, TSharedPtr<FOnlineSessionSearchResult>* InDesiredSession)
		: PlayerNum(InPlayerNum)
		, SessionName(InSessionName)
		, DesiredSession(InDesiredSession)
		, bUseOverload(true)
	{}

	
	FSessionJoinSessionStep(FUniqueNetIdPtr* InPlayerId,const FName& InSessionName, TSharedPtr<FOnlineSessionSearchResult>* InDesiredSession)
		: PlayerId(InPlayerId)
		, SessionName(InSessionName)
		, DesiredSession(InDesiredSession)
	{}

	virtual ~FSessionJoinSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnJoinSessionCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnJoinSessionCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, JoinSessionCall, JoinSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnJoinSessionCompleteDelegate JoinSessionsDelegate = FOnJoinSessionCompleteDelegate::CreateLambda([this](FName InSessionName, EOnJoinSessionCompleteResult::Type InResult)
		{
			REQUIRE(State == EState::JoinSessionCalled);
			CHECK(SessionName == InSessionName);
			CHECK(InResult == EOnJoinSessionCompleteResult::Type::Success);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			OnJoinSessionsCompleteDelegateHandle = OnlineSessionPtr->AddOnJoinSessionCompleteDelegate_Handle(JoinSessionsDelegate);

			State = EState::JoinSessionCall;
			break;
		}
		case EState::JoinSessionCall:
		{
			State = EState::JoinSessionCalled;

			bool Result = false;
			if (bUseOverload)
			{
				Result = OnlineSessionPtr->JoinSession(PlayerNum, SessionName, *DesiredSession->Get());
			}
			else
			{
				Result = OnlineSessionPtr->JoinSession(*PlayerId->Get(), SessionName, *DesiredSession->Get());
			}
			REQUIRE(Result == true);
			break;
		}
		case EState::JoinSessionCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnJoinSessionCompleteDelegate_Handle(OnJoinSessionsCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	int32 PlayerNum = 0;
	FUniqueNetIdPtr* PlayerId = nullptr;
	FName SessionName = TEXT("");
	TSharedPtr<FOnlineSessionSearchResult>* DesiredSession = nullptr;
	bool bUseOverload = false;
	FDelegateHandle	OnJoinSessionsCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};