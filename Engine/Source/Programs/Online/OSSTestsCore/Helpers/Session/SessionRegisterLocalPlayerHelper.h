// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionRegisterLocalPlayerStep : public FTestPipeline::FStep
{
	FSessionRegisterLocalPlayerStep(FUniqueNetIdPtr* InPlayer, FName InSessionName, EOnJoinSessionCompleteResult::Type InExpectedSessionCompleteType)
		: TestSessionName(InSessionName)
		, Player(InPlayer)
		, ExpectedSessionCompleteType(InExpectedSessionCompleteType)
	{}

	virtual ~FSessionRegisterLocalPlayerStep() = default;

	enum class EState { Init, RegisterLocalPlayerCall, RegisterLocalPlayerCalled, ClearDelegates, Done } State = EState::Init;

	FOnRegisterLocalPlayerCompleteDelegate RegisterLocalPlayerDelegate = FOnRegisterLocalPlayerCompleteDelegate::CreateLambda([this](const FUniqueNetId& InPlayer, EOnJoinSessionCompleteResult::Type InJoinSessionComplete)
		{
			REQUIRE(State == EState::RegisterLocalPlayerCalled);
			CHECK(*Player->Get() == InPlayer);
			CHECK(EOnJoinSessionCompleteResult::Type::Success == InJoinSessionComplete);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			State = EState::RegisterLocalPlayerCall;
			break;
		}
		case EState::RegisterLocalPlayerCall:
		{
			State = EState::RegisterLocalPlayerCalled;

			OnlineSessionPtr->RegisterLocalPlayer(*Player->Get(), TestSessionName, RegisterLocalPlayerDelegate);
			break;
		}
		case EState::RegisterLocalPlayerCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	FUniqueNetIdPtr* Player = nullptr;
	EOnJoinSessionCompleteResult::Type ExpectedSessionCompleteType = EOnJoinSessionCompleteResult::UnknownError;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};