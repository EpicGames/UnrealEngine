// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionRegisterPlayerStep : public FTestPipeline::FStep
{
	FSessionRegisterPlayerStep(FName InSessionName, FUniqueNetIdPtr* InPlayer, bool bInWasInvited = false)
		: TestSessionName(InSessionName)
		, Player(InPlayer)
		, bWasInvited(bInWasInvited)
	{}

	virtual ~FSessionRegisterPlayerStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnRegisterPlayersCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnRegisterPlayersCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, RegisterPlayerCall, RegisterPlayerCalled, ClearDelegates, Done } State = EState::Init;

	FOnRegisterPlayersCompleteDelegate RegisterPlayerDelegate = FOnRegisterPlayersCompleteDelegate::CreateLambda([this](FName SessionName, const TArray<FUniqueNetIdRef>& Players,bool bWasSuccessful)
		{
			REQUIRE(State == EState::RegisterPlayerCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
				REQUIRE(OnlineSessionPtr != nullptr);

				OnRegisterPlayerCompleteDelegateHandle = OnlineSessionPtr->AddOnRegisterPlayersCompleteDelegate_Handle(RegisterPlayerDelegate);

				State = EState::RegisterPlayerCall;
				break;
			}
			case EState::RegisterPlayerCall:
			{
				State = EState::RegisterPlayerCalled;
			
				bool Result = OnlineSessionPtr->RegisterPlayer(TestSessionName, *Player->Get(), bWasInvited);
				REQUIRE(Result == true);
				break;
			}
			case EState::RegisterPlayerCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineSessionPtr->ClearOnRegisterPlayersCompleteDelegate_Handle(OnRegisterPlayerCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	FUniqueNetIdPtr* Player = nullptr;
	bool bWasInvited = false;
	FDelegateHandle	OnRegisterPlayerCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};