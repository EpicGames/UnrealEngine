// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionRegisterPlayersStep : public FTestPipeline::FStep
{
	FSessionRegisterPlayersStep(FName InSessionName, TArray<FUniqueNetIdRef>* InPlayers, bool bInWasInvited = false)
		: TestSessionName(InSessionName)
		, Players(InPlayers)
		, bWasInvited(bInWasInvited)
	{}

	virtual ~FSessionRegisterPlayersStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnRegisterPlayersCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnRegisterPlayersCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, RegisterPlayersCall, RegisterPlayersCalled, ClearDelegates, Done } State = EState::Init;

	FOnRegisterPlayersCompleteDelegate RegisterPlayersDelegate = FOnRegisterPlayersCompleteDelegate::CreateLambda([this](FName SessionName, const TArray<FUniqueNetIdRef>& InPlayers,bool bWasSuccessful)
		{
			REQUIRE(State == EState::RegisterPlayersCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
				REQUIRE(OnlineSessionPtr != nullptr);

				OnRegisterPlayersCompleteDelegateHandle = OnlineSessionPtr->AddOnRegisterPlayersCompleteDelegate_Handle(RegisterPlayersDelegate);

				State = EState::RegisterPlayersCall;
				break;
			}
			case EState::RegisterPlayersCall:
			{
				State = EState::RegisterPlayersCalled;

				bool Result = OnlineSessionPtr->RegisterPlayers(TestSessionName, *Players, bWasInvited);
				REQUIRE(Result == true);
				break;
			}
			case EState::RegisterPlayersCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineSessionPtr->ClearOnRegisterPlayersCompleteDelegate_Handle(OnRegisterPlayersCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	TArray<FUniqueNetIdRef>* Players = nullptr;
	bool bWasInvited = false;
	FDelegateHandle	OnRegisterPlayersCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};