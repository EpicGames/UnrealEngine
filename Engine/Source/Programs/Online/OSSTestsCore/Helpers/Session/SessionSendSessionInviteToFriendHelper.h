// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionSendSessionInviteToFriendStep : public FTestPipeline::FStep
{
	FSessionSendSessionInviteToFriendStep(FUniqueNetIdPtr* InLocalUserId, const FName& InSessionName, FUniqueNetIdPtr* InFriendId)
		: LocalUserId(InLocalUserId)
		, FriendId(InFriendId)
		, SessionName(InSessionName)
	{}

	FSessionSendSessionInviteToFriendStep(int32 InLocalUserNum, const FName& InSessionName, FUniqueNetIdPtr* InFriendId)
		: LocalUserNum(InLocalUserNum)
		, FriendId(InFriendId)
		, SessionName(InSessionName)
		, bUseOverload(true)
	{}
	
	virtual ~FSessionSendSessionInviteToFriendStep() = default;
	
	enum class EState { Init, SendSessionInviteToFriendCall, SendSessionInviteToFriendCalled, ClearDelegates, Done } State = EState::Init;
	
	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);
			
			State = EState::SendSessionInviteToFriendCall;
			break;
		}
		case EState::SendSessionInviteToFriendCall:
		{
			State = EState::SendSessionInviteToFriendCalled;
			bool Result = false;
			if (bUseOverload)
			{
				Result = OnlineSessionPtr->SendSessionInviteToFriend(LocalUserNum, SessionName, *FriendId->Get());
			}
			else
			{
				Result = OnlineSessionPtr->SendSessionInviteToFriend(*LocalUserId->Get(), SessionName, *FriendId->Get());
			}
			
			/*the Result should be true, but at the moment it function returns false anyway. Jira task: OI-4260*/
			REQUIRE(Result == true);
			break;
		}
		case EState::SendSessionInviteToFriendCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}
		return EContinuance::ContinueStepping;
	}
protected:
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr* LocalUserId = nullptr;
	FUniqueNetIdPtr* FriendId = nullptr;
	FName SessionName = TEXT("");
	bool bUseOverload = false;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};