// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionStartSessionStep : public FTestPipeline::FStep
{
	FSessionStartSessionStep(FName InSessionName)
		: SessionName(InSessionName)
	{}

	virtual ~FSessionStartSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnStartSessionCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnStartSessionCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, StartSessionCall, StartSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnStartSessionCompleteDelegate StartSessionDelegate = FOnStartSessionCompleteDelegate::CreateLambda([this](FName InSessionName, bool bWasSuccessful)
		{
			REQUIRE(State == EState::StartSessionCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == InSessionName.ToString());
	
			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
				REQUIRE(OnlineSessionPtr != nullptr);

				OnStartSessionCompleteDelegateHandle = OnlineSessionPtr->AddOnStartSessionCompleteDelegate_Handle(StartSessionDelegate);

				State = EState::StartSessionCall;
				break;
			}
			case EState::StartSessionCall:
			{
				State = EState::StartSessionCalled;

				bool Result = OnlineSessionPtr->StartSession(SessionName);
				REQUIRE(Result == true);
				break;
			}
			case EState::StartSessionCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineSessionPtr->ClearOnStartSessionCompleteDelegate_Handle(OnStartSessionCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName SessionName = {};
	FDelegateHandle	OnStartSessionCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};