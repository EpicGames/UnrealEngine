// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionUnregisterLocalPlayerStep : public FTestPipeline::FStep
{
	FSessionUnregisterLocalPlayerStep(FUniqueNetIdPtr* InPlayer, FName InSessionName)
		: TestSessionName(InSessionName)
		, Player(InPlayer)
	{}

	virtual ~FSessionUnregisterLocalPlayerStep() = default;

	enum class EState { Init, UnregisterLocalPlayerCall, UnregisterLocalPlayerCalled, ClearDelegates, Done } State = EState::Init;

	FOnUnregisterLocalPlayerCompleteDelegate UnregisterLocalPlayerDelegate = FOnUnregisterLocalPlayerCompleteDelegate::CreateLambda([this](const FUniqueNetId& InPlayer, const bool bWasSuccessful)
		{
			REQUIRE(State == EState::UnregisterLocalPlayerCalled);
			CHECK(*Player->Get() == InPlayer);
			CHECK(bWasSuccessful);

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			State = EState::UnregisterLocalPlayerCall;
			break;
		}
		case EState::UnregisterLocalPlayerCall:
		{
			State = EState::UnregisterLocalPlayerCalled;

			OnlineSessionPtr->UnregisterLocalPlayer(*Player->Get(), TestSessionName, UnregisterLocalPlayerDelegate);
			break;
		}
		case EState::UnregisterLocalPlayerCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	FUniqueNetIdPtr* Player = nullptr;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};