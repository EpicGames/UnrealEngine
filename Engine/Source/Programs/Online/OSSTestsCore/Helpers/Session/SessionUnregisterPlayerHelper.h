// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionUnregisterPlayerStep : public FTestPipeline::FStep
{
	FSessionUnregisterPlayerStep(FName InSessionName, FUniqueNetIdPtr* InPlayer)
		: TestSessionName(InSessionName)
		, Player(InPlayer)
	{}

	virtual ~FSessionUnregisterPlayerStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnUnregisterPlayersCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnUnregisterPlayersCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, UnregisterPlayerCall, UnregisterPlayerCalled, ClearDelegates, Done } State = EState::Init;

	FOnUnregisterPlayersCompleteDelegate UnregisterPlayerDelegate = FOnUnregisterPlayersCompleteDelegate::CreateLambda([this](FName SessionName, const TArray<FUniqueNetIdRef>& Players, bool bWasSuccessful)
		{
			REQUIRE(State == EState::UnregisterPlayerCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			OnUnregisterPlayerCompleteDelegateHandle = OnlineSessionPtr->AddOnUnregisterPlayersCompleteDelegate_Handle(UnregisterPlayerDelegate);

			State = EState::UnregisterPlayerCall;
			break;
		}
		case EState::UnregisterPlayerCall:
		{
			State = EState::UnregisterPlayerCalled;

			bool Result = OnlineSessionPtr->UnregisterPlayer(TestSessionName, *Player->Get());
			REQUIRE(Result == true);

			break;
		}
		case EState::UnregisterPlayerCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnUnregisterPlayersCompleteDelegate_Handle(OnUnregisterPlayerCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	FUniqueNetIdPtr* Player = nullptr;
	FDelegateHandle	OnUnregisterPlayerCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};