// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"

struct FSessionUnregisterPlayersStep : public FTestPipeline::FStep
{
	FSessionUnregisterPlayersStep(FName InSessionName, TArray<FUniqueNetIdRef>* InPlayers)
		: TestSessionName(InSessionName)
		, Players(InPlayers)
	{}

	virtual ~FSessionUnregisterPlayersStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnUnregisterPlayersCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnUnregisterPlayersCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, UnregisterPlayersCall, UnregisterPlayersCalled, ClearDelegates, Done } State = EState::Init;

	FOnUnregisterPlayersCompleteDelegate UnregisterPlayersDelegate = FOnUnregisterPlayersCompleteDelegate::CreateLambda([this](FName SessionName, const TArray<FUniqueNetIdRef>& InPlayers, bool bWasSuccessful)
		{
			REQUIRE(State == EState::UnregisterPlayersCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
			REQUIRE(OnlineSessionPtr != nullptr);

			OnUnregisterPlayersCompleteDelegateHandle = OnlineSessionPtr->AddOnUnregisterPlayersCompleteDelegate_Handle(UnregisterPlayersDelegate);

			State = EState::UnregisterPlayersCall;
			break;
		}
		case EState::UnregisterPlayersCall:
		{
			State = EState::UnregisterPlayersCalled;

			bool Result = OnlineSessionPtr->UnregisterPlayers(TestSessionName, *Players);
			REQUIRE(Result == true);

			break;
		}
		case EState::UnregisterPlayersCalled:
		{
			break;
		}
		case EState::ClearDelegates:
		{
			OnlineSessionPtr->ClearOnUnregisterPlayersCompleteDelegate_Handle(OnUnregisterPlayersCompleteDelegateHandle);

			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	TArray<FUniqueNetIdRef>* Players = nullptr;
	FDelegateHandle	OnUnregisterPlayersCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};