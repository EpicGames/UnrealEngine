// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Interfaces/OnlineSessionInterface.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"

struct FSessionUpdateSessionStep : public FTestPipeline::FStep
{
	FSessionUpdateSessionStep(FName InSessionName, FOnlineSessionSettings& InUpdatedSessionSettings, bool bInShouldRefreshOnlineData)
		: TestSessionName(InSessionName)
		, NewSessionSettings(InUpdatedSessionSettings)
		, bShouldRefreshOnlineData(bInShouldRefreshOnlineData)
	{}

	virtual ~FSessionUpdateSessionStep()
	{
		if (OnlineSessionPtr != nullptr)
		{
			if (OnlineSessionPtr->OnUpdateSessionCompleteDelegates.IsBound())
			{
				OnlineSessionPtr->OnUpdateSessionCompleteDelegates.Clear();
			}
			OnlineSessionPtr = nullptr;
		}
	};

	enum class EState { Init, UpdateSessionCall, UpdateSessionCalled, ClearDelegates, Done } State = EState::Init;

	FOnUpdateSessionCompleteDelegate UpdateSessionDelegate = FOnUpdateSessionCompleteDelegate::CreateLambda([this](FName SessionName, bool bWasSuccessful)
		{
			REQUIRE(State == EState::UpdateSessionCalled);
			CHECK(bWasSuccessful);
			CHECK(SessionName.ToString() == TestSessionName.ToString());

			State = EState::ClearDelegates;
		});

	virtual EContinuance Tick(IOnlineSubsystem* OnlineSubsystem) override
	{
		switch (State)
		{
			case EState::Init:
			{
				OnlineSessionPtr = OnlineSubsystem->GetSessionInterface();
				REQUIRE(OnlineSessionPtr != nullptr);

				OnUpdateSessionCompleteDelegateHandle = OnlineSessionPtr->AddOnUpdateSessionCompleteDelegate_Handle(UpdateSessionDelegate);

				State = EState::UpdateSessionCall;
				break;
			}
			case EState::UpdateSessionCall:
			{
				State = EState::UpdateSessionCalled;

				bool Result = OnlineSessionPtr->UpdateSession(TestSessionName, NewSessionSettings, bShouldRefreshOnlineData);
				REQUIRE(Result == true);

				break;
			}
			case EState::UpdateSessionCalled:
			{
				break;
			}
			case EState::ClearDelegates:
			{
				OnlineSessionPtr->ClearOnUpdateSessionCompleteDelegate_Handle(OnUpdateSessionCompleteDelegateHandle);

				State = EState::Done;
				break;
			}
			case EState::Done:
			{
				return EContinuance::Done;
			}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	FName TestSessionName = {};
	FOnlineSessionSettings NewSessionSettings = {};
	bool bShouldRefreshOnlineData = false;
	FDelegateHandle	OnUpdateSessionCompleteDelegateHandle;
	IOnlineSessionPtr OnlineSessionPtr = nullptr;
};