// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System.Linq;
using System;

public class OSSTestsCore : ModuleRules
{
	public OSSTestsCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.Add("Programs/Online/OSSTestsCore");

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Projects",
				"EngineSettings",
				"OnlineSubsystem",
				"OnlineSubsystemNull",
				"ApplicationCore"
			}
		);

		// Disable external auth if target doesn't define it.
		if (!Target.GlobalDefinitions.Contains("OSSTESTS_USEEXTERNAUTH=1"))
		{
			PublicDefinitions.Add(String.Format("OSSTESTS_USEEXTERNAUTH=0"));
		}
	}
}
