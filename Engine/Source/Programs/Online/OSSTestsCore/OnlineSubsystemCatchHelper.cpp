// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemCatchHelper.h"

#include "Algo/AllOf.h"
#include "Algo/Sort.h"
#include "Algo/ForEach.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Helpers/Identity/IdentityAutoLoginHelper.h"
#include "Helpers/Identity/IdentityLoginHelper.h"
#include "Helpers/Identity/IdentityLogoutHelper.h"
#include "Helpers/TickForTime.h"

#include "OnlineSubsystemNames.h"
#include "Misc/CommandLine.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

#include "Online/CoreOnline.h"

// Make sure there are registered input devices for N users and fire
// OnInputDeviceConnectionChange delegate for interested online service code.
void EnsureLocalUserCount(uint32 NumUsers)
{
	TArray<FPlatformUserId> Users;
	IPlatformInputDeviceMapper::Get().GetAllActiveUsers(Users);

	const uint32 ActiveUserCount = Users.Num();

	for (uint32 Index = ActiveUserCount; Index < NumUsers; ++Index)
	{
		int32 Result = Users.Find(FPlatformMisc::GetPlatformUserForUserIndex(Index));
		if (Result < 0)
		{
			IPlatformInputDeviceMapper::Get().Internal_MapInputDeviceToUser(
				FInputDeviceId::CreateFromInternalId(Index),
				FPlatformMisc::GetPlatformUserForUserIndex(Index),
				EInputDeviceConnectionState::Connected);
		}
	}
}

TArray<TFunction<void()>>* GetGlobalInitalizers()
{
	static TArray<TFunction<void()>> gInitalizersToCallInMain;
	return &gInitalizersToCallInMain;
}

TArray<FString> GetServiceModules()
{
	TArray<FString> Modules;

	for (const FOnlineSubsystemFixtureInvoker::FApplicableServicesConfig& Config : FOnlineSubsystemFixtureInvoker::GetApplicableServices())
	{
		for (const FString& Module : Config.ModulesToLoad)
		{
			Modules.AddUnique(Module);
		}
	}

	return Modules;
}

void FOnlineSubsystemTestBaseFixture::LoadServiceModules()
{
	for (const FString& Module : GetServiceModules())
	{
		FModuleManager::LoadModulePtr<IModuleInterface>(*Module);
	}
}

void FOnlineSubsystemTestBaseFixture::UnloadServiceModules()
{
	const TArray<FString>& Modules = GetServiceModules();
	// Shutdown in reverse order
	for (int Index = Modules.Num() - 1; Index >= 0; --Index)
	{
		if (IModuleInterface* Module = FModuleManager::Get().GetModule(*Modules[Index]))
		{
			Module->ShutdownModule();
		}
	}
}

void FOnlineSubsystemTestBaseFixture::DestroyCurrentOnlineSubsystemModule() const
{
	FName SubsystemName = FName(GetSubsystem());
	FModuleManager& ModuleManager = FModuleManager::Get();
	FName ModuleName = SubsystemName;
	const bool bIsShutdown = false;
	ModuleManager.UnloadModule(ModuleName, bIsShutdown);
	FModuleManager::LoadModulePtr<IModuleInterface>(ModuleName);
}

void FOnlineSubsystemTestBaseFixture::ConstructInternal(FString SubsystemName)
{
	Subsystem = SubsystemName;
}

FOnlineSubsystemTestBaseFixture::FOnlineSubsystemTestBaseFixture()
	: Driver()
	, Pipeline(Driver.MakePipeline())
{
	// handle most cxn in ConstructInternal
}

FOnlineSubsystemTestBaseFixture::~FOnlineSubsystemTestBaseFixture()
{
}

FString FOnlineSubsystemTestBaseFixture::GetSubsystem() const
{
	return Subsystem;
}

IOnlineSubsystem* FOnlineSubsystemTestBaseFixture::GetSubsystemPtr() const
{
	return IOnlineSubsystem::Get(FName(Subsystem));
	//return UE::Online::FOnlineServicesRegistry::Get().GetNamedServicesInstance(ServiceType, NAME_None, NAME_None);
}

TArray<FOnlineAccountCredentials> FOnlineSubsystemTestBaseFixture::GetIniCredentials(int32 TestAccountIndex) const
{
	FString LoginCredentialCategory = FString::Printf(TEXT("LoginCredentials %s"), *Subsystem);
	TArray<FString> CredentialsArr;
	GConfig->GetArray(*LoginCredentialCategory, TEXT("Credentials"), CredentialsArr, GEngineIni);

	if (TestAccountIndex > CredentialsArr.Num())
	{
		UE_LOGF(LogOSSTests, Error, "Attempted to GetCredentials for more than we have stored! Add more credentials to the DefaultEngine.ini for OssTests");
		return TArray<FOnlineAccountCredentials>();
	}

	TArray<FOnlineAccountCredentials> OnlineAccountCredentials;
	for (int32 Index = 0; Index < CredentialsArr.Num(); ++Index)
	{
		FString LoginUsername, LoginType, LoginToken;
		FParse::Value(*CredentialsArr[Index], TEXT("Type="), LoginType);
		FParse::Value(*CredentialsArr[Index], TEXT("Id="), LoginUsername);
		FParse::Value(*CredentialsArr[Index], TEXT("Token="), LoginToken);
		INFO(*FString::Printf(TEXT("Logging in with type %s, id %s, password %s"), *LoginType, *LoginUsername, *LoginToken));

		OnlineAccountCredentials.Add(FOnlineAccountCredentials{ LoginType, LoginUsername, LoginToken });
	}
	return OnlineAccountCredentials;
}

TArray<FOnlineAccountCredentials> FOnlineSubsystemTestBaseFixture::GetCredentials(int32 TestAccountIndex, int32 NumUsers) const
{
#if OSSTESTS_USEEXTERNAUTH
	return CustomCredentials(TestAccountIndex, NumUsers);
#else // OSSTESTS_USEEXTERNAUTH
	return GetIniCredentials(TestAccountIndex);
#endif // OSSTESTS_USEEXTERNAUTH
}

FString FOnlineSubsystemTestBaseFixture::GetLoginCredentialCategory() const
{
	return FString::Printf(TEXT("LoginCredentials %s"), *Subsystem);
}

void FOnlineSubsystemTestBaseFixture::AssignLoginUsers(int32 LocalUserNum, FUniqueNetIdPtr& OutAccountId) const
{
	IOnlineSubsystem* OnlineSubsystem = GetSubsystemPtr();
	IOnlineIdentityPtr OnlineIdentityPtr = OnlineSubsystem->GetIdentityInterface();
	FUniqueNetIdPtr UserId = OnlineIdentityPtr->GetUniquePlayerId(LocalUserNum);
	CHECK(UserId != nullptr);
	OutAccountId = UserId;
}

FTestPipeline& FOnlineSubsystemTestBaseFixture::GetLoginPipeline(uint32 NumUsersToLogin, bool MultiLogin) const
{
	if (!MultiLogin)
	{
		REQUIRE(NumLocalUsers == -1); // Don't call GetLoginPipeline more than once per test
	}
	NumLocalUsers = NumUsersToLogin;
	NumUsersToLogout = NumUsersToLogin;

	bool bUseAutoLogin = false;
	bool bUseImplicitLogin = false;
	FString LoginCredentialCategory = FString::Printf(TEXT("LoginCredentials %s"), *Subsystem);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseAutoLogin"), bUseAutoLogin, GEngineIni);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseImplicitLogin"), bUseImplicitLogin, GEngineIni);

	// Make sure input delegates are fired for adding the required user count.
	EnsureLocalUserCount(NumUsersToLogin);

	if (bUseImplicitLogin)
	{
		// Users are expected to already be valid.
	}
	else if (bUseAutoLogin)
	{
		NumLocalUsers = 1;
		Pipeline.EmplaceStep<FIdentityAutoLoginStep>(0);
	}
	else
	{
		TArray<FOnlineAccountCredentials> AuthLoginParams = GetCredentials(0, NumUsersToLogin);

		for (uint32 Index = 0; Index < NumUsersToLogin; ++Index)
		{
			Pipeline.EmplaceStep<FIdentityLoginStep>(Index, AuthLoginParams[Index]);
		}
	}

	return Pipeline;
}

FTestPipeline& FOnlineSubsystemTestBaseFixture::GetLoginPipeline(uint32 TestAccountIndex, std::initializer_list<std::reference_wrapper<FUniqueNetIdPtr>> AccountIds) const
{
	REQUIRE(NumLocalUsers == -1); // Don't call GetLoginPipeline more than once per test
	
	NumLocalUsers = AccountIds.size();

	bool bUseAutoLogin = false;
	bool bUseImplicitLogin = false;
	FString LoginCredentialCategory = FString::Printf(TEXT("LoginCredentials %s"), *Subsystem);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseAutoLogin"), bUseAutoLogin, GEngineIni);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseImplicitLogin"), bUseImplicitLogin, GEngineIni);

	// Make sure input delegates are fired for adding the required user count.
	EnsureLocalUserCount(NumLocalUsers);

	if (bUseImplicitLogin)
	{
		// Users are expected to already be valid.
		int32 Index = 0;
		for (FUniqueNetIdPtr& AccountId : AccountIds)
		{
			AssignLoginUsers(Index, AccountId);
			++Index;
		}
	}
	else if (bUseAutoLogin)
	{
		NumLocalUsers = 1;
		Pipeline.EmplaceStep<FIdentityAutoLoginStep>(0);
	}
	else if (NumLocalUsers > 0)
	{
		TArray<FOnlineAccountCredentials> AuthLoginParams = GetCredentials(TestAccountIndex, NumLocalUsers);

		int32 Index = 0;
		for (FOnlineAccountCredentials AuthLoginParam : AuthLoginParams)
		{
			Pipeline.EmplaceStep<FIdentityLoginStep>(Index, AuthLoginParams[Index]);
			++Index;
		}

		// Perform login so we can bulk assign users in the next step.
		RunToCompletion(false);

		Index = 0;
		for (FUniqueNetIdPtr& AccountId : AccountIds)
		{
			AssignLoginUsers(Index, AccountId);
			++Index;
		}
	}

	return Pipeline;
}

FTestPipeline& FOnlineSubsystemTestBaseFixture::GetLoginPipeline(std::initializer_list<std::reference_wrapper<FUniqueNetIdPtr>> AccountIds) const
{
	return GetLoginPipeline(0, AccountIds);
}

FTestPipeline& FOnlineSubsystemTestBaseFixture::GetPipeline() const
{
	return GetLoginPipeline(0);
}

void FOnlineSubsystemTestBaseFixture::RunToCompletion(bool bLogout, bool bWaitBeforeLogout, const FTimespan TimeToWaitMilliseconds, const FString SubsystemInstanceName) const
{
	bool bUseAutoLogin = false;
	bool bUseImplicitLogin = false;
	FString LoginCredentialCategory = FString::Printf(TEXT("LoginCredentials %s"), *Subsystem);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseAutoLogin"), bUseAutoLogin, GEngineIni);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseImplicitLogin"), bUseImplicitLogin, GEngineIni);

	if (bLogout)
	{
		if (bUseImplicitLogin)
		{
			// Users are expected to already be valid.
		}
		else if (bUseAutoLogin)
		{
			NumLocalUsers = 1;
			Pipeline.EmplaceStep<FIdentityAutoLoginStep>(0);
		}
		else if (NumLocalUsers > 0)
		{
			for (uint32 i = 0; i < NumLocalUsers; i++)
			{
				if (bWaitBeforeLogout)
				{
					Pipeline
						.EmplaceStep<FTickForTime>(TimeToWaitMilliseconds);
				}

				Pipeline.EmplaceStep<FIdentityLogoutStep>(i);
			}
		}
	}
	
	
	FName SubsystemName = FName(GetSubsystem());
	FPipelineTestContext TestContext = FPipelineTestContext(Subsystem, SubsystemInstanceName);
	CHECK(Driver.AddPipeline(MoveTemp(Pipeline), TestContext));
	REQUIRE(IOnlineSubsystem::IsEnabled(SubsystemName));
	Driver.RunToCompletion();
}

void FOnlineSubsystemFixtureInvoker::SetSubsystem(const FString& InStoredSubsystem) 
{
	StoredSubsystem = InStoredSubsystem;
}

TArray<FOnlineSubsystemFixtureInvoker::FApplicableServicesConfig> FOnlineSubsystemFixtureInvoker::GetApplicableServices()
{
	static TArray<FApplicableServicesConfig> ServicesConfig =
		[]()
		{
			TArray<FApplicableServicesConfig> ServicesConfigInit;
			if (const TCHAR* CmdLine = FCommandLine::Get())
			{
				FString Values;
				TArray<FString> ServicesTags;
				if (FParse::Value(CmdLine, TEXT("-Services="), Values, false))
				{
					Values.ParseIntoArray(ServicesTags, TEXT(","));
				}

				if (ServicesTags.IsEmpty())
				{
					GConfig->GetArray(TEXT("OnlineServicesTests"), TEXT("DefaultServices"), ServicesTags, GEngineIni);
				}

				for (const FString& ServicesTag : ServicesTags)
				{
					FString ConfigCategory = FString::Printf(TEXT("OnlineServicesTests %s"), *ServicesTag);
					FApplicableServicesConfig Config;
					Config.Tag = ServicesTag;

					FString ServicesType;
					GConfig->GetString(*ConfigCategory, TEXT("ServicesType"), ServicesType, GEngineIni);
					GConfig->GetArray(*ConfigCategory, TEXT("ModulesToLoad"), Config.ModulesToLoad, GEngineIni);

					LexFromString(Config.ServicesType, *ServicesType);
					if (Config.ServicesType != UE::Online::EOnlineServices::None)
					{
						ServicesConfigInit.Add(MoveTemp(Config));
					}
				}
			}

			return ServicesConfigInit;
		}();

	return ServicesConfig;
}

TArray<FOnlineSubsystemFixtureInvoker::FApplicableSubsystemConfig> FOnlineSubsystemFixtureInvoker::GetApplicableSubsystems()
{
	static TArray<FApplicableSubsystemConfig> SubsystemsConfig =
		[]()
		{
			TArray<FApplicableSubsystemConfig> SubsystemsConfigInit;
			if (const TCHAR* CmdLine = FCommandLine::Get())
			{
				FString Values;
				TArray<FString> SubsystemsNames;
				if (FParse::Value(CmdLine, TEXT("-Subsystems="), Values, false))
				{
					Values.ParseIntoArray(SubsystemsNames, TEXT(","));
				}

				if (SubsystemsNames.IsEmpty())
				{
					GConfig->GetArray(TEXT("OnlineSubsystemTests"), TEXT("Subsystems"), SubsystemsNames, GEngineIni);
				}

				for (const FString& SubsystemName : SubsystemsNames)
				{
					FApplicableSubsystemConfig Config;
					Config.Name = SubsystemName;
					SubsystemsConfigInit.Add(MoveTemp(Config));
				}
			}

			return SubsystemsConfigInit;
		}();

	return SubsystemsConfig;
}

bool FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(const TArray<FString>& TestTags, const TArray<FString>& InputTags)
{
	if (InputTags.Num() == 0)
	{
		return false;
	}

	if (InputTags.Num() > TestTags.Num())
	{
		return false;
	}

	bool bAllInputTagsInTestTags = Algo::AllOf(InputTags, [&TestTags](const FString& CheckTag) -> bool
		{
			auto CheckStringCaseInsenstive = [&CheckTag](const FString& TestString) -> bool
			{
				return TestString.Equals(CheckTag, ESearchCase::IgnoreCase);
			};

			if (TestTags.ContainsByPredicate(CheckStringCaseInsenstive))
			{
				return true;
			}

			return false;
		});

	return bAllInputTagsInTestTags;
}

bool FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(const TArray<FString>& TestTags, const FString& RawTagString)
{
	TArray<FString> InputTags;
	RawTagString.ParseIntoArray(InputTags, TEXT(","));
	Algo::ForEach(InputTags, [](FString& String)
		{
			String.TrimStartAndEndInline();
			String.RemoveFromStart("[");
			String.RemoveFromEnd("]");
		});
	return CheckAllTagsIsIn(TestTags, InputTags);
}

FString FOnlineSubsystemFixtureInvoker::GenerateTags(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, const TCHAR* InTag)
{
	//Copy String here for ease-of-manipulation
	FString RawInTag = InTag;

	TArray<FString> TestTagsArray;
	RawInTag.ParseIntoArray(TestTagsArray, TEXT("]"));
	Algo::ForEach(TestTagsArray, [](FString& String)
		{
			String.TrimStartAndEndInline();
			String.RemoveFromStart("[");
		});
	Algo::Sort(TestTagsArray);

	// Search if we need to append [!mayfail] tag to indicate to 
	// catch2 this test is in a in-development phase and failures 
	// should be ignored.
	for (const FString& FailableTags : SkippableTags.MayFailTags)
	{
		if (CheckAllTagsIsIn(TestTagsArray, FailableTags))
		{
			RawInTag.Append(TEXT("[!mayfail]"));
			break;
		}
	}

	// Search if we need to append [!shouldfail] tag to indicate to 
	// catch2 this test should fail, and if it ever passes we should
	// should fail.
	for (const FString& FailableTags : SkippableTags.ShouldFailTags)
	{
		if (CheckAllTagsIsIn(TestTagsArray, FailableTags))
		{
			RawInTag.Append(TEXT("[!shouldfail]"));
			break;
		}
	}

	return FString::Printf(TEXT("[%s] %s"), *ServiceName, *RawInTag);
}

FOnlineSubsystemFixtureInvoker::EDisableReason FOnlineSubsystemFixtureInvoker::ShouldDisableTest(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, FString& InTag)
{
	TArray<FString> TestTagsArray;
	InTag.ParseIntoArray(TestTagsArray, TEXT("]"));
	Algo::ForEach(TestTagsArray, [](FString& String)
		{
			String.TrimStartAndEndInline();
			String.RemoveFromStart("[");
		});
	Algo::Sort(TestTagsArray);

	// If we contain [!<service>] it means we shouldn't run this
	// test against this service.
	if (InTag.Contains("!" + ServiceName))
	{
		return EDisableReason::AgainstService;
	}

	// Check for exclusive runs
	for (const FApplicableServicesConfig& Config : GetApplicableServices())
	{
		const FString& ServiceTag = Config.Tag;
		if (ServiceName.Equals(ServiceTag, ESearchCase::IgnoreCase))
		{
			if (InTag.Contains("." + ServiceTag))
			{
				FString SubstringToRemove = TEXT(".");
				FString Replacement = TEXT("");
				InTag.ReplaceInline(*SubstringToRemove, *Replacement);
			}
			
			continue;
		}

		// If we contain [.NULL] and we're running with [EOS] we shouldn't
		// generate a test for [EOS] here.
		if (InTag.Contains("." + ServiceTag))
		{
			return EDisableReason::ExclusiveService;
		}
	}

	// If we contain tags from config it means 
	// we shouldn't run this test
	for (const FString& DisableTag : SkippableTags.DisableTestTags)
	{
		if (CheckAllTagsIsIn(TestTagsArray, DisableTag))
		{
			return EDisableReason::DisableTagPresence;
		}
	}

	// We should run the test!
	return EDisableReason::Success;
}

FOnlineSubsystemFixtureInvoker::EDisableReason FOnlineSubsystemFixtureInvoker::ShouldSkipTest(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, FString& InTag) {

	// If we have tags present indicating we should exit the test
	if (EDisableReason Reason = ShouldDisableTest(ServiceName, SkippableTags, InTag); Reason != EDisableReason::Success)
	{
		return Reason;
	}

	return EDisableReason::Success;
}

bool FOnlineSubsystemFixtureInvoker::IsRunningTestSkipOnTags(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, FString& InTag)
{
	bool bIsShouldSkip = true;
	EDisableReason Reason = ShouldSkipTest(ServiceName, SkippableTags, InTag);

	switch (Reason)
	{
		case EDisableReason::Success:
		{
			bIsShouldSkip = false;
			break;
		}
		case EDisableReason::AgainstService:
		{
			UE_LOGF(LogOSSTests, Verbose, "Test skipped due to run against this service.");
			break;
		}
		case EDisableReason::ExclusiveService:
		{
			UE_LOGF(LogOSSTests, Verbose, "Test skipped due to exclusive service run.");
			break;
		}
		case EDisableReason::DisableTagPresence:
		{
			UE_LOGF(LogOSSTests, Verbose, "Test skipped due to disable tag presence.");
			break;
		}
		default:
		{
			UE_LOGF(LogOSSTests, Error, "Test skipped due to unknown reason!");
			break;
		}
	}
			
	return bIsShouldSkip;
}

// This code is kept identical to Catch internals so that there is as little deviation from OSS_TESTS and Online_OSS_TESTS as possible
FOnlineSubsystemAutoReg::FOnlineSubsystemAutoReg(Catch::Detail::unique_ptr<FOnlineSubsystemFixtureInvoker> TestInvoker, Catch::SourceLineInfo LineInfo, const char* Name, const char* Tags, const char* AddlOnlineInfo)
{
	auto TestInvokerPtr = TestInvoker.release();
	auto GlobalInitalizersPtr = GetGlobalInitalizers();
	ensure(GlobalInitalizersPtr);
	GlobalInitalizersPtr->Add([TestInvokerPtr = MoveTemp(TestInvokerPtr), LineInfo, Name, Tags, this]() mutable -> void
		{
			for (const FOnlineSubsystemFixtureInvoker::FApplicableSubsystemConfig& Config : FOnlineSubsystemFixtureInvoker::GetApplicableSubsystems())
			{
				const FString SubsystemName = Config.Name;
				FString ReportingCategory = FString::Printf(TEXT("TestReporting %s"), *SubsystemName);
				FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkippableTags;
				GConfig->GetArray(*ReportingCategory, TEXT("MayFailTestTags"), SkippableTags.MayFailTags, GEngineIni);
				GConfig->GetArray(*ReportingCategory, TEXT("ShouldFailTestTags"), SkippableTags.ShouldFailTags, GEngineIni);
				GConfig->GetArray(*ReportingCategory, TEXT("DisableTestTags"), SkippableTags.DisableTestTags, GEngineIni);

				auto NewName = StringCast<ANSICHAR>(*FString::Printf(TEXT("[%s] %s"), *SubsystemName, ANSI_TO_TCHAR(Name)));
				auto GeneratedTags = StringCast<ANSICHAR>(*FOnlineSubsystemFixtureInvoker::GenerateTags(SubsystemName, SkippableTags, ANSI_TO_TCHAR(Tags)));
				FString NewTags = GeneratedTags.Get();

				auto ClonedTestInvokerPtr = TestInvokerPtr->clone();

				if (!FOnlineSubsystemFixtureInvoker::IsRunningTestSkipOnTags(SubsystemName, SkippableTags, NewTags))
				{
					ClonedTestInvokerPtr->SetSubsystem(SubsystemName);
				}
				else
				{
					continue;
				}

				Catch::getMutableRegistryHub().registerTest(Catch::makeTestCaseInfo(
					std::string(Catch::StringRef()),  // Used for testing a static method instead of a function- not needed since we're passing an ITestInvoker macro
					Catch::NameAndTags{ NewName.Get(), TCHAR_TO_ANSI(*NewTags) },
					LineInfo),
					Catch::Detail::unique_ptr(ClonedTestInvokerPtr.release()) // This is taking the ITestInvoker macro and will call invoke() to run the test
				);

			}

			delete TestInvokerPtr;
		});
}

