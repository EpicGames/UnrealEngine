// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TestHarness.h"
#include "TestDriver.h"
#include "CoreMinimal.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "Containers/Ticker.h"

#include <catch2/catch_test_case_info.hpp>
#include <catch2/interfaces/catch_interfaces_registry_hub.hpp>

class FOnlineSubsystemFixtureInvoker : public Catch::ITestInvoker
{
public:

	struct FReportingSkippableTags
	{
		TArray<FString> MayFailTags;
		TArray<FString> ShouldFailTags;
		TArray<FString> DisableTestTags;
	};

	struct FApplicableServicesConfig
	{
		FString Tag;
		UE::Online::EOnlineServices ServicesType;
		TArray<FString> ModulesToLoad;
	};

	struct FApplicableSubsystemConfig
	{
		FString Name;
	};

	enum class EDisableReason { Success, AgainstService, ExclusiveService, DisableTagPresence };

	virtual void invoke() const = 0;

	virtual Catch::Detail::unique_ptr<FOnlineSubsystemFixtureInvoker> clone() = 0;

	void SetSubsystem(const FString& InStoredSubsystem);

	// TODO: This should poll some info from the tags and generate the list based on that
	static TArray<FApplicableServicesConfig> GetApplicableServices();
	static TArray<FApplicableSubsystemConfig> GetApplicableSubsystems();

	/*
	* Helper function that calls CheckAllTagsIsIn(const TArray<FString>&, const TArray<FString>&);
	*
	* @param  TestTags			The array of test tags we wish to test against.
	* @param  RawTagString		Comma seperated string of elemnets we wish to convert to an array.
	*
	* @return true if all elements of RawTagString prased as an comma sperated array is in TestTags.
	*/
	static bool CheckAllTagsIsIn(const TArray<FString>& TestTags, const FString& RawTagString);

	/**
	 * Checks if every element of InputTags is in TestTags.
	 *
	 * @param  TestTags       The array of test tags we wish to test against.
	 * @param  InputTags	  The array of tags we wish to see if all elements of are present in TestTags.
	 *
	 * @return  true if all elements of InputTags is in TestTags.
	 */
	static bool CheckAllTagsIsIn(const TArray<FString>& TestTags, const TArray<FString>& InputTags);

	static FString GenerateTags(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, const TCHAR* InTag);

	static EDisableReason ShouldDisableTest(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, FString& InTag);
	static EDisableReason ShouldSkipTest(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, FString& InTag);
	static bool IsRunningTestSkipOnTags(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, FString& InTag);

protected:
	FString StoredSubsystem;
};

template <typename C>
class TestInvokerFixture : public FOnlineSubsystemFixtureInvoker
{
public:
	constexpr TestInvokerFixture(void (C::* InTestAsMethod)()) noexcept 
		: TestAsMethod(InTestAsMethod) 
	{
	}

	void PrepareTestCase() const
	{
		Fixture = Catch::Detail::make_unique<C>();
		Fixture->ConstructInternal(StoredSubsystem);
	}

	void TearDownTestCase() const
	{
		Fixture.reset();
	}

	void invoke() const override
	{
		PrepareTestCase();
		auto* f = Fixture.get();
		(f->*TestAsMethod)();
		TearDownTestCase();
	}

	Catch::Detail::unique_ptr<FOnlineSubsystemFixtureInvoker> clone() override
	{
		return Catch::Detail::unique_ptr<FOnlineSubsystemFixtureInvoker>(Catch::Detail::make_unique<TestInvokerFixture<C>>(TestAsMethod));
	}

private:
	void (C::* TestAsMethod)();
	mutable Catch::Detail::unique_ptr<C> Fixture = nullptr;
};

template<typename C>
Catch::Detail::unique_ptr<FOnlineSubsystemFixtureInvoker> MakeTestInvokerFixture(void (C::* TestAsMethod)()) {
	return Catch::Detail::make_unique<TestInvokerFixture<C>>(TestAsMethod);
}

class FOnlineSubsystemTestBaseFixture : public Catch::ITestInvoker
{
public:
	void ConstructInternal(FString ServiceName);
	virtual ~FOnlineSubsystemTestBaseFixture();

	/* Loads all necessary services for the current test run */
	static void LoadServiceModules();

	/* Unloads all necessary services for the current test run */
	static void UnloadServiceModules();

protected:

	FOnlineSubsystemTestBaseFixture();
	
	FString GetSubsystem() const;

	IOnlineSubsystem* GetSubsystemPtr() const;

	virtual void DestroyCurrentOnlineSubsystemModule() const;

#if OSSTESTS_USEEXTERNAUTH
	TArray<FOnlineAccountCredentials> CustomCredentials(int32 TestAccountIndex, int32 NumUsers) const;
#endif // OSSTESTS_USEEXTERNAUTH

	TArray<FOnlineAccountCredentials> GetIniCredentials(int32 TestAccountIndex) const;

	TArray<FOnlineAccountCredentials> GetCredentials(int32 TestAccountIndex, int32 NumUsers) const;

	/* Attempts to assign OutAccountId to what LocalUserId is logged in as. */
	void AssignLoginUsers(int32 LocalUserNum, FUniqueNetIdPtr& OutAccountId) const;

	FTestPipeline& GetLoginPipeline(uint32 NumUsersToLogin = 1, bool MultiLogin = false) const;
	FTestPipeline& GetLoginPipeline(uint32 TestAccountIndex, std::initializer_list<std::reference_wrapper<FUniqueNetIdPtr>> AccountIds) const;
	FTestPipeline& GetLoginPipeline(std::initializer_list<std::reference_wrapper<FUniqueNetIdPtr>> AccountIds) const;
	FTestPipeline& GetPipeline() const;

	/* Returns the ini login category name for the configured service */
	FString GetLoginCredentialCategory() const;

	void RunToCompletion(bool bLogout = true, bool bWaitBeforeLogout = false, const FTimespan TimeToWaitMilliseconds = FTimespan::FromMilliseconds(1000), const FString SubsystemInstanceName = TEXT("")) const;

	/* ITestInvoker */
	virtual void invoke() const override = 0;

private:
	FString Tags;
	FString Subsystem;
	// Catch's ITestInvoker is a const interface but we'll be changing stuff (emplacing steps into the driver, setting flags, etc.) so we make these mutable
	mutable FTestDriver Driver;
	mutable FTestPipeline Pipeline;
	mutable uint32 NumLocalUsers = -1;
	mutable uint32 NumUsersToLogout = -1;
};

class FOnlineSubsystemAutoReg
{
public:
	// This code is kept identical to Catch internals so that there is as little deviation from OSS_TESTS and Online_OSS_TESTS as possible
	FOnlineSubsystemAutoReg(Catch::Detail::unique_ptr<FOnlineSubsystemFixtureInvoker> TestInvoker, Catch::SourceLineInfo LineInfo, const char* Name, const char* Tags, const char* AddlOnlineInfo);
};

TArray<TFunction<void()>>* GetGlobalInitalizers();

#define INTERNAL_ONLINESUBSYSTEM_TEST_CASE_NAMED_FIXTURE(RegName, ClassName, Name, Tags, ...)\
namespace {\
class UE_JOIN(OnlineSubsystemTest_,RegName) : public ClassName {\
public:\
	virtual void invoke() const override;\
	void test();\
};\
	FOnlineSubsystemAutoReg RegName( ::MakeTestInvokerFixture( UE_JOIN(&OnlineSubsystemTest_,RegName)::test ), CATCH_INTERNAL_LINEINFO, Name, Tags, "");\
}\
void UE_JOIN(OnlineSubsystemTest_,RegName)::test()\
{\
	invoke();\
}\
void UE_JOIN(OnlineSubsystemTest_,RegName)::invoke() const\

#define ONLINESUBSYSTEM_TEST_CASE_FIXTURE(ClassName, Name, Tags, ...) \
	INTERNAL_ONLINESUBSYSTEM_TEST_CASE_NAMED_FIXTURE(INTERNAL_CATCH_UNIQUE_NAME(OnlineSubsystemRegistrar), ClassName, Name, Tags, __VA_ARGS__)

#define ONLINESUBSYSTEM_TEST_CASE(Name, Tags, ...) \
	INTERNAL_ONLINESUBSYSTEM_TEST_CASE_NAMED_FIXTURE(INTERNAL_CATCH_UNIQUE_NAME(OnlineSubsystemRegistrar), FOnlineSubsystemTestBaseFixture, Name, Tags, __VA_ARGS__)

#define REQUIRE_OP(Op)\
	CAPTURE(Op);\
	REQUIRE(Op.WasSuccessful());
