// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestHarness.h"
#include "Containers/Ticker.h"

FString FTestPipeline::InfoString() const
{
	double CurrentTime = FPlatformTime::Seconds();

	TStringBuilder<128> Info;
	Info.Append(TEXT("TestSteps:"));
	Info.Append(FString::FromInt(TestSteps.Num()));
	Info.Append(TEXT(" "));
	Info.Append(TEXT("CompletedSteps:"));
	Info.Append(FString::FromInt(CompletedSteps.Num()));
	Info.Append(TEXT(" "));
	Info.Append(TEXT("TimedoutSteps:"));
	Info.Append(FString::FromInt(TimedoutSteps.Num()));
	Info.Append(TEXT(" ")),
	Info.Append(TEXT("DeletePostReleaseSteps:"));
	Info.Append(FString::FromInt(DeletePostReleaseSteps.Num()));
	Info.Append(TEXT(" "));
	Info.Append(TEXT("PipelineTime:"));
	Info.Append(FString::FromInt(FTimespan(CurrentTime - PipelineStartTime).GetTotalSeconds()));
	Info.Append(TEXT("s "));
	Info.Append(TEXT("SubsystemTickSum:"));
	Info.Append(FString::FromInt(FTimespan(SubsystemTickSum).GetTotalMilliseconds()));
	Info.Append(TEXT("ms "));
	Info.Append(TEXT("SubsystemTickCount:"));
	Info.Append(FString::FromInt(SubsystemTickCount));

	const FString InfoString = Info.ToString();

	return InfoString;
}

FString FTestDriver::TimeoutFailedTestInfo() const
{
	TStringBuilder<128> FailedInfo;
	FailedInfo.Append(TEXT("[Error] Test driver encountered a timeout during test execution."));
	FailedInfo.Append(TEXT("   TestFailedOnStepNum:"));
	FailedInfo.Append(FString::FromInt(this->FailedStepNum));

	const FString FailedInfoString = FailedInfo.ToString();

	return FailedInfoString;
}

void FTestPipeline::operator()(IOnlineSubsystem* Subsystem)
{
	double CurrentTime = FPlatformTime::Seconds();

	if (TestSteps.IsEmpty())
	{
		Driver.MarkComplete(Subsystem);
		return;
	}

	FStepPtr& CurrentStep = TestSteps.HeapTop();

	if (CurrentStep->StepWaitsForCompletion())
	{
		if (CheckStepTimedOut(CurrentStep, CurrentTime, Subsystem))
		{
			return;
		}

		if (CurrentStep->Tick(Subsystem) == FStep::EContinuance::Done)
		{
			CompletedSteps.Emplace(MoveTemp(CurrentStep));
			TestSteps.RemoveAt(0);
		}
		else
		{
			return; // Wait until state is Done
		}
	}
	else
	{
		switch (CurrentStep->Tick(Subsystem))
		{

		case FStep::EContinuance::Done:
			// Retain the completed steps for those which have long term notify handlers.
			CompletedSteps.Emplace(MoveTemp(CurrentStep));
			TestSteps.RemoveAt(0);
			break;

		default:

			if (CheckStepTimedOut(CurrentStep, CurrentTime, Subsystem))
			{
				return;
			}
			break;
		}
	}
}

bool FTestPipeline::CheckStepTimedOut(FStepPtr& CurrentStep, double CurrentTime, IOnlineSubsystem* Subsystem)
{
	if (FTimespan::FromSeconds(CurrentTime - PipelineStartTime).GetTicks() >= PipelineTimeout.GetTicks())
	{
		Driver.SetDriverTimedOut(true);
		if (CurrentStep->IsOptional())
		{
			// Mark only this step as timed out.
			// Retain the timed-out steps since the callback has yet to be handled.
			TimedoutSteps.Emplace(MoveTemp(CurrentStep));
			TestSteps.RemoveAt(0);
		}
		else
		{
			// Mark all remaining steps as timed out.
			while (!TestSteps.IsEmpty())
			{
				FTestPipeline::FStepPtr& ExistingStep = TestSteps.HeapTop();
				// Retain the timed-out steps since the callback may not have been handled.
				TimedoutSteps.Emplace(MoveTemp(ExistingStep));
				TestSteps.RemoveAt(0);
			}
			Driver.MarkComplete(Subsystem);
			return true;
		}
	}
	return false;
}

void FTestPipeline::OnPreRelease()
{
	if(!TestSteps.IsEmpty() || !TimedoutSteps.IsEmpty())
	{
		this->Driver.FailedStepNum = CompletedSteps.Num()+1;
	}

	auto ReleaseList = [this](TArray<FStepPtr>& Steps)
	{
		while (!Steps.IsEmpty())
		{
			FTestPipeline::FStepPtr& ExistingStep = Steps.Top();
			if (ExistingStep->RequiresDeletePostRelease())
			{
				ExistingStep->OnPreRelease();
				DeletePostReleaseSteps.Emplace(MoveTemp(ExistingStep));
			}
			Steps.RemoveAt(0);
		}
	};

	ReleaseList(TestSteps);
	ReleaseList(CompletedSteps);
	ReleaseList(TimedoutSteps);
}

void FTestPipeline::EvaluatePlatformTickTime(double&& TickTime)
{
	SubsystemTickSum += FTimespan::FromSeconds(TickTime).GetTicks();
	++SubsystemTickCount;

	if (EvaluateTickConfig.bEvaluateTickCheckActive)
	{
		if (SubsystemTickCount >= EvaluateTickConfig.MinimumTickCount)
		{
			FTimespan AverageTick = SubsystemTickSum.GetTicks() / SubsystemTickCount;
			FTimespan ExpectedAverageTick = EvaluateTickConfig.ExpectedAverageTick.GetTicks();
			//CHECK(AverageTick <= ExpectedAverageTick);
		}

		FTimespan Tick = FTimespan::FromSeconds(TickTime);
		FTimespan AbsoluteMaximumTick = EvaluateTickConfig.AbsoluteMaximumTick.GetTicks();
		//CHECK(Tick <= AbsoluteMaximumTick);
	}
}

FTestDriver::~FTestDriver()
{
	// Mark all remaining subsystems as complete.
	ForeachSubsystemInstance([this](IOnlineSubsystem* OnlineSubsystem, FTestPipeline&)
		{
			MarkComplete(OnlineSubsystem);
		});

	FlushCompleted();
}

bool FTestDriver::AddPipeline(FTestPipeline&& Pipeline, const FPipelineTestContext& TestContext)
{
	FString CompleteName = TestContext.InitOptions.InstanceName.IsEmpty()
		? TestContext.InitOptions.SubsystemType
		: TestContext.InitOptions.SubsystemType + TEXT(":") + TestContext.InitOptions.InstanceName;
	IOnlineSubsystem* OnlineSubsystem = IOnlineSubsystem::Get(FName(CompleteName));
	if (OnlineSubsystem == nullptr)
	{
		return false;
	}
	else
	{
		SubsystemInstances.Emplace(OnlineSubsystem, MoveTemp(Pipeline));
		return true;
	}
}

void FTestDriver::MarkComplete(IOnlineSubsystem* Key)
{
	CompletedInstances.Emplace(Key);
}

void FTestDriver::RunToCompletion()
{
	LastTickTime = FPlatformTime::Seconds();

	ForeachSubsystemInstance([](IOnlineSubsystem* Subsystem, FTestPipeline& Func)
		{
			Func.Start();
		});

	while (!SubsystemInstances.IsEmpty())
	{
		FPlatformProcess::Sleep(TICK_DURATION.GetTotalSeconds());
		
			ForeachSubsystemInstance([this](IOnlineSubsystem* Subsystem, FTestPipeline& CurrentPipeline)
				{
					const double BeforeTick = FPlatformTime::Seconds();
					{ // Scope for INFO

						double TickTime = BeforeTick - LastTickTime;
						FTSTicker::GetCoreTicker().Tick(TickTime);
						LastTickTime = BeforeTick;
					}

					const double AfterTick = FPlatformTime::Seconds();
					CurrentPipeline.EvaluatePlatformTickTime(AfterTick - BeforeTick);
				});
		ForeachSubsystemInstance([](IOnlineSubsystem* Subsystem, FTestPipeline& Func)
			{
				//UE_LOGF(LogOSSTests, Log, "INFO %ls", *Func.InfoString());
				Func(Subsystem);
			});
		FlushCompleted();
	}

	if (bDidTimeout)
	{
		FAIL_CHECK(TCHAR_TO_ANSI (*TimeoutFailedTestInfo()));
	}
}

void FTestDriver::FlushCompleted()
{
	for (IOnlineSubsystem* Key : CompletedInstances)
	{
		SubsystemInstances.Find(Key)->OnPreRelease();
		SubsystemInstances.Remove(Key);
	}
	CompletedInstances.Empty();
}