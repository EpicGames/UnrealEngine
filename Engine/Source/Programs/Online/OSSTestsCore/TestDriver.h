// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "OnlineSubsystem.h"
#include "OnlineSubsystemNames.h"

#ifndef UE_BUILD_DEBUG
#define UE_BUILD_DEBUG 0
#endif

DEFINE_LOG_CATEGORY_STATIC(LogOSSTests, Log, Log);

#define UE_LOG_OSSTESTS(Verbosity, Format, ...) \
{ \
	UE_LOGF(LogOSSTests, Verbosity, "OSSTests: %ls", *FString::Printf(Format, ##__VA_ARGS__)); \
}

static const FTimespan TICK_DURATION = FTimespan::FromMilliseconds(1);

class FTestDriver;

struct FPipelineTestContextInitOptions
{
	FString SubsystemType;
	FString InstanceName;
};

class FPipelineTestContext
{

public:
	FPipelineTestContextInitOptions InitOptions;

	FPipelineTestContext(const FString& SubsystemType, const FString& InstanceName = TEXT(""))
	{
		InitOptions.SubsystemType = SubsystemType;
		InitOptions.InstanceName = InstanceName;
	}

};

#define INFO_TEST_STEP() INFO("Test Step Index: " << Index)
#define INFO_TEST_STEP_OF(InTestStep) INFO("Test Step Index: " << InTestStep->Index)

class FTestPipeline
{
public:
	/**
	 * Defines the configuration used when checking for excess tick lengths.
	 * The default setup is relatively lenient.  We should tighten this up
	 * then determine the specific test cases that are allowed to deviate.
	 */
	struct FEvaluateTickConfig
	{
		/** If true then the excess tick check will be used. */
		bool bEvaluateTickCheckActive = true;
		/**
		 * The expected average tick length.
		 * After the MinimumTickCount ticks, if the average tick exceeds this value then a CHECK failure will occurr.
		 * The default is changed for debug.
		 */
		FTimespan ExpectedAverageTick = FTimespan::FromMilliseconds(5);
		/**
		 * The absolute maximum tick length.
		 * If the current tick exceeds this value then a CHECK failure will occurr.
		 * The default is changed for debug.
		 */
		FTimespan AbsoluteMaximumTick = FTimespan::FromMilliseconds(75);
		/**
		 * The minimum number of ticks per test before the average tick is evaluated against ExpectedAvergateTick.
		 * The default is changed for debug.
		 */
		uint32_t MinimumTickCount = 10;

		/**
		 * Basic constructor used as the default with enabled tick check and settings,
		 * or when disabling the tick check.
		 */
		FEvaluateTickConfig(bool bInEvaluateTickCheckActive = true)
			: bEvaluateTickCheckActive(bInEvaluateTickCheckActive)
#if defined(UE_BUILD_DEBUG) && UE_BUILD_DEBUG
			, ExpectedAverageTick(FTimespan::FromMilliseconds(10))
			, AbsoluteMaximumTick(FTimespan::FromMilliseconds(150))
			, MinimumTickCount(10)
#endif
		{}

		/** Full constructor which allows changing all settings for an enabled tick check. */
		FEvaluateTickConfig(FTimespan&& InExpectedAverageTick, FTimespan&& InAbsoluteMaximumTick, uint32_t InMinimumTickCount = 10)
			: bEvaluateTickCheckActive(true)
			, ExpectedAverageTick(MoveTemp(InExpectedAverageTick))
			, AbsoluteMaximumTick(MoveTemp(InAbsoluteMaximumTick))
			, MinimumTickCount(InMinimumTickCount)
		{
		}
	};

	class FStep
	{
	public:
		uint32_t Index = 0;
		enum class EContinuance { ContinueStepping, Done };
		virtual ~FStep() {}
		virtual bool IsOptional() const { return false; }
		// Wait to complete before next steps can Tick
		bool StepWaitsForCompletion() const { return WaitForCompletion; }
		virtual bool RequiresDeletePostRelease() const { return false; }
		// Called only if RequestDeletePostRelease is true and the FStep is being stored.
		virtual void OnPreRelease() {}
		virtual EContinuance Tick(IOnlineSubsystem* Subsystem) = 0;
	protected:
		bool WaitForCompletion = false; // wait before next steps can Tick
	};
	using FStepPtr = TUniquePtr<FStep>;

	class FLambdaStep : public FStep
	{
	public:
		FLambdaStep(TUniqueFunction<void(IOnlineSubsystem*)>&& InLambda)
			: Lambda(MoveTemp(InLambda))
		{}

		virtual EContinuance Tick(IOnlineSubsystem* Subsystem) override
		{
			Lambda(Subsystem);
			return EContinuance::Done;
		}

	private:
		TUniqueFunction<void(IOnlineSubsystem*)> Lambda;
	};

	FTestPipeline(FTestPipeline&&) = default;
	FTestPipeline(const FTestPipeline&) = delete;

	/** Adds an overall test timeout value to the pipeline. */
	FTestPipeline&& WithTimeout(FTimespan&& InTimeout)
	{
		PipelineTimeout = MoveTemp(InTimeout);
		return MoveTemp(*this);
	}

	/** Disables the per tick timer checks. */
	FTestPipeline&& WithoutEvaluateTickCheck()
	{
		EvaluateTickConfig = FEvaluateTickConfig(false);
		return MoveTemp(*this);
	}

	/** Changes and enables the per tick timer checks based on the provided arguments. */
	FTestPipeline&& WithEvaluateTickCheck(FTimespan&& InExpectedAverageTick, FTimespan&& InAbsoluteMaximumTick, uint32_t InMinimumTickCount = 10)
	{
		EvaluateTickConfig = FEvaluateTickConfig(MoveTemp(InExpectedAverageTick), MoveTemp(InAbsoluteMaximumTick), InMinimumTickCount);
		return MoveTemp(*this);
	}

	/** Adds a test step. */
	template <typename T, typename... TArguments>
	FTestPipeline&& EmplaceStep(TArguments&&... Args)
	{
		static_assert(std::is_base_of_v<FStep, T>, "Step type must be derived from FTestPipeline::FStep");
		uint32_t NewIndex = (uint32_t)TestSteps.Num();
		TestSteps.Emplace(MakeUnique<T>(Forward<TArguments>(Args)...));
		TestSteps.Last()->Index = NewIndex;

		return MoveTemp(*this);
	}

	FTestPipeline&& EmplaceLambda(TUniqueFunction<void(IOnlineSubsystem*)> Lambda)
	{
		return EmplaceStep<FLambdaStep>(MoveTemp(Lambda));
	}

	/** Generates a string suitable for INFO which will help identify where and when a specific failure has occurred. */
	FString InfoString() const;

	void operator()(IOnlineSubsystem* Subsystem);

	void OnPreRelease();

	/** Given the time points before Tick and after, perform the configured excess tick check. */
	void EvaluatePlatformTickTime(const FTimespan& InDuration)
	{
		//EvaluatePlatformTickTime(InDuration);
	}

	void Start()
	{
		PipelineStartTime = FPlatformTime::Seconds();
	}

private:
	FTestPipeline(FTestDriver& InDriver, FTimespan&& InTimeout)
		: Driver(InDriver)
		, PipelineTimeout(MoveTemp(InTimeout))
	{
	}

	FTestPipeline(FTestDriver& InDriver, FTimespan&& InTimeout, FEvaluateTickConfig&& InEvaluateTickConfig)
		: Driver(InDriver)
		, PipelineTimeout(MoveTemp(InTimeout))
		, EvaluateTickConfig(MoveTemp(InEvaluateTickConfig))
	{
	}

	/** Given the duration of the call to Tick, perform the configured excess tick check. */
	void EvaluatePlatformTickTime(double&& TickTime);
	bool CheckStepTimedOut(FStepPtr& Step, double CurrentTime, IOnlineSubsystem* Subsystem);


	FTestDriver& Driver;
	TArray<FStepPtr> TestSteps;
	TArray<FStepPtr> CompletedSteps;
	TArray<FStepPtr> TimedoutSteps;
	TArray<FStepPtr> DeletePostReleaseSteps;
	FTimespan PipelineTimeout;
	double PipelineStartTime;
	/** Current excess tick config. */
	FEvaluateTickConfig EvaluateTickConfig;
	/** Sum of time to process all calls to Subsystem Tick. */
	FTimespan SubsystemTickSum = FTimespan::FromMilliseconds(0);;
	/** Number of calls to Subsystem Tick. */
	uint32 SubsystemTickCount = 0;

	friend class FTestDriver;
};

class FTestDriver
{
public:
	using FSubsystemInstanceMap = TMap<IOnlineSubsystem*, FTestPipeline>;

	~FTestDriver();

	template <typename Func>
	void ForeachSubsystemInstance(Func&& Fn)
	{
		for (auto& [Subsystem, DriverFunc] : SubsystemInstances)
		{
			Fn(Subsystem, DriverFunc);
		}
	}

	FTestPipeline MakePipeline()
	{
		return MakePipeline(FTimespan::FromSeconds(60));
	}

	template <typename... TArgs>
	FTestPipeline MakePipeline(TArgs&&... Args)
	{
		return FTestPipeline(*this, Forward<TArgs>(Args)...);
	}

	bool AddPipeline(FTestPipeline&& Pipeline, const FPipelineTestContext& TestContext = FPipelineTestContext(NULL_SUBSYSTEM.ToString()));

	void MarkComplete(IOnlineSubsystem* Key);

	void RunToCompletion();

	void SetDriverTimedOut(bool InValue) { UE_DEBUG_BREAK(); bDidTimeout = InValue; };

	FString TimeoutFailedTestInfo() const;

	int32 FailedStepNum = 0;

private:

	void FlushCompleted();

	FSubsystemInstanceMap SubsystemInstances;
	TSet<IOnlineSubsystem*> CompletedInstances;
	bool bDidTimeout = false;
	double LastTickTime = 0.0;
};


