// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TestHarness.h"

class FTestUtilities
{
public:
	static FString GetUniqueTestString()
	{
		int32 Rand = FMath::RandHelper(INT_MAX);
		FString UniqueString;
		UniqueString.Append(FString::FromInt(Rand));

		return UniqueString;
	}

	static int32 GetUniqueTestInteger()
	{
		int32 Rand = FMath::RandHelper(INT_MAX);
		
		return Rand;
	}
};
