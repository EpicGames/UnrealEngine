// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "Interfaces/OnlineIdentityInterface.h"
#include "OnlineSubsystem.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/TickForTime.h"
#include "Helpers/Identity/IdentityLogoutHelper.h"

#define ACCOUNTSERVICE_TAG "[AccountService]"

#define ACCOUNTSERVICE_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, ACCOUNTSERVICE_TAG __VA_ARGS__)

ACCOUNTSERVICE_TEST_CASE("Verify if we can properly create a OnlineAccountCredentials object")
{
	// FOnlineAccountCredentials(const FString& InType, const FString& InId, const FString& InToken) :
	FString LocalType = "test_account";
	FString LocalId = "12345";
	FString LocalToken = "fake_token";
	FOnlineAccountCredentials LocalAccount = FOnlineAccountCredentials(LocalType, LocalId, LocalToken);
	CHECK(&LocalAccount != nullptr);
}

ACCOUNTSERVICE_TEST_CASE("Verify if we can properly instantiate the OSS")
{
	int32 LocalUserNum = 0;
	int32 NumUsers = 1;

	GetPipeline()
		.EmplaceLambda([this, &LocalUserNum, &NumUsers](IOnlineSubsystem* Services)
		{
			TArray<FOnlineAccountCredentials> AccountCreds = GetCredentials(LocalUserNum, NumUsers);

			IOnlineIdentityPtr IdentityInterface = Services->GetIdentityInterface();		
			bool LoggedIn = IdentityInterface.Get()->Login(LocalUserNum, AccountCreds[0]);
			REQUIRE(LoggedIn);
		})
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FIdentityLogoutStep>(LocalUserNum);

	RunToCompletion();
}