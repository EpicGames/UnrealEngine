// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Achievements/AchievementsQueryAchievementDescriptionsHelper.h"
#include "Helpers/Achievements/AchievementsGetCachedAchievementDescriptionHelper.h"

#include "Helpers/TickForTime.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_GETCACHEDACHIEMENTDESCRIPTION_TAG ACHIEVEMENTS_TAG "[getcachedachievementdescription]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Verify calling Achievements GetCachedAchievementDescription with valid inputs returns the expected result(Success Case)", EG_ACHIEVEMENTS_GETCACHEDACHIEMENTDESCRIPTION_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FString AchievementId = TEXT("");
	int32 NumUsersToImplicitLogin = 1;
	bool bLogout = true;
	bool bWaitBeforeLogout = true;

	if (GetSubsystem() == "EOS")
	{
		AchievementId = TEXT("test_getachievementdefinitioncount");
	}
	else if(GetSubsystem() == "NULL")
	{
		AchievementId = TEXT("null-ach-0");
	}

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FAchievementsQueryAchievementDescriptionsStep>(&LocalUserId)
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FAchievementsGetCachedAchievementDescriptionStep>(AchievementId);
		
	RunToCompletion(bLogout, bWaitBeforeLogout);
}
