// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Achievements/AchievementsQueryAchievementDescriptionsHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_QUERYACHIEVEMENTDESCRIPTIONS_TAG ACHIEVEMENTS_TAG "[queryachievementdescriptions]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Verify calling Achievements QueryAchievementDescriptions with valid inputs returns the expected result(Success Case)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTDESCRIPTIONS_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 NumUsersToImplicitLogin = 1;
	bool bLogout = true;
	bool bWaitBeforeLogout = true;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FAchievementsQueryAchievementDescriptionsStep>(&LocalUserId);

	RunToCompletion(bLogout, bWaitBeforeLogout);
}
