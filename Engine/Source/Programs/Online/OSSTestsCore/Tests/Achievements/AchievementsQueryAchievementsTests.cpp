// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Achievements/AchievementsQueryAchievementsHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_QUERYACHIEVEMENTS_TAG ACHIEVEMENTS_TAG "[queryachievements]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Verify calling Achievements QueryAchievements with valid inputs returns the expected result(Success Case)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTS_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 NumUsersToImplicitLogin = 1;
	bool bLogout = true;
	bool bWaitBeforeLogout = true;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FAchievementsQueryAchievementsStep>(&LocalUserId);

	RunToCompletion(bLogout, bWaitBeforeLogout);
}
