// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Achievements/AchievementsQueryAchievementsHelper.h"
#include "Helpers/Achievements/AchievementsWriteAchievementsHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_WRITEACHIEVEMENTS_TAG ACHIEVEMENTS_TAG "[writeachievements]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Verify calling Achievements WriteAchievements with valid inputs returns the expected result(Success Case)", EG_ACHIEVEMENTS_WRITEACHIEVEMENTS_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FOnlineAchievementsWritePtr AchievementWriteObject = MakeShareable(new FOnlineAchievementsWrite());
	FOnlineAchievementsWriteRef AchievementWriter = AchievementWriteObject.ToSharedRef();

	double CurrentTimeSeconds = FPlatformTime::Seconds();
	FString TimestampString = FString::Printf(TEXT("%f"), CurrentTimeSeconds);

	AchievementWriter->Properties.Emplace("test_unlockachievements", 1);

	int32 NumUsersToImplicitLogin = 1;
	bool bLogout = true;
	bool bWaitBeforeLogout = true;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FAchievementsQueryAchievementsStep>(&LocalUserId)
		.EmplaceStep<FAchievementsWriteAchievementsStep>(&LocalUserId, AchievementWriter);

	RunToCompletion(bLogout, bWaitBeforeLogout);
}
