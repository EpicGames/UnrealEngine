// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Friends/FriendsEnsureFriendshipHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Friends/FriendsBlockPlayerHelper.h"
#include "Helpers/Friends/FriendsUnblockPlayerHelper.h"
#include "Helpers/Friends/FriendsGetBlockedPlayersHelper.h"
#include "Helpers/Friends/FriendsQueryBlockedPlayersHelper.h"

#define FRIENDS_TAG "[suite_friends]"
#define EG_FRIENDS_BLOCKPLAYER_TAG FRIENDS_TAG "[blockplayer]"

#define FRIENDS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, FRIENDS_TAG __VA_ARGS__)

FRIENDS_TEST_CASE("Verify calling BlockPlayer with valid inputs returns the expected result(Success Case)", EG_FRIENDS_BLOCKPLAYER_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 TargetUserNum = 1;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FString LocalListName = EFriendsLists::ToString(EFriendsLists::Default);
	bool bLocalIsBlockedPlayersListPopulated = true;
	bool bLocalIsFriendsListPopulated = true;
	int32 NumUsersToImplicitLogin = 2;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FFriendsEnsureFriendshipStep>(LocalUserNum, TargetUserNum, &LocalUserId, &TargetUserId, LocalListName, bLocalIsFriendsListPopulated)
		.EmplaceStep<FFriendsBlockPlayerStep>(LocalUserNum, &TargetUserId)
		.EmplaceStep<FFriendsQueryBlockedPlayersStep>(&LocalUserId, bLocalIsBlockedPlayersListPopulated)
		.EmplaceStep<FFriendsGetBlockedPlayersStep>(&LocalUserId, &TargetUserId)
		.EmplaceStep<FFriendsUnblockPlayerStep>(LocalUserNum, &TargetUserId);

	RunToCompletion();
}
