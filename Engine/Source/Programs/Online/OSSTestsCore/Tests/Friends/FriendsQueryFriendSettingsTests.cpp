// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Friends/FriendsEnsureFriendshipHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Friends/FriendsQueryFriendSettingsHelper.h"
#include "Helpers/Friends/FriendsGetFriendSettingsHelper.h"
#include "Helpers/Friends/FriendsSetFriendSettingsHelper.h"

#define FRIENDS_TAG "[suite_friends]"
#define EG_FRIENDS_QUERYFRIENDSETTINGS_TAG FRIENDS_TAG "[queryfriendsettings]"

#define FRIENDS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, FRIENDS_TAG __VA_ARGS__)

FRIENDS_TEST_CASE("Verify calling QueryFriendSettings with valid inputs returns the expected result(Success Case)", EG_FRIENDS_QUERYFRIENDSETTINGS_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 TargetUserNum = 1;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FString LocalListName = EFriendsLists::ToString(EFriendsLists::Default);
	bool bLocalIsFriendsListPopulated = true;
	bool bLocalNeverShowAgain = true;
	const FString LocalSource = TEXT("Steam");
	int32 NumUsersToImplicitLogin = 2;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FFriendsEnsureFriendshipStep>(LocalUserNum, TargetUserNum, &LocalUserId, &TargetUserId, LocalListName, bLocalIsFriendsListPopulated)
		.EmplaceStep<FFriendsSetFriendSettingsStep>(&TargetUserId, LocalSource, bLocalNeverShowAgain)
		.EmplaceStep<FFriendsQueryFriendSettingsStep>(&TargetUserId, LocalSource)
		.EmplaceStep<FFriendsGetFriendSettingsStep>(&TargetUserId, LocalSource);

	RunToCompletion();
}
