// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Friends/FriendsEnsureFriendshipHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Friends/FriendsQueryFriendSettingsHelper.h"
#include "Helpers/Friends/FriendsGetFriendInvitePolicyHelper.h"
#include "Helpers/Friends/FriendsUpdateFriendInvitePolicySettingsHelper.h"

#define FRIENDS_TAG "[suite_friends]"
#define EG_FRIENDS_UPDATEFRIENDINVITEPOLICYSETTINGS_TAG FRIENDS_TAG "[updatefriendinvitepolicysettings]"

#define FRIENDS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, FRIENDS_TAG __VA_ARGS__)

FRIENDS_TEST_CASE("Verify calling UpdateFriendInvitePolicySettings with valid inputs returns the expected result(Success Case)", EG_FRIENDS_UPDATEFRIENDINVITEPOLICYSETTINGS_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 TargetUserNum = 1;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FString LocalListName = EFriendsLists::ToString(EFriendsLists::Default);
	bool bLocalInAffectsExistingInvites = true;
	bool LocalIsFriendsListPopulated = true;
	int32 NumUsersToImplicitLogin = 2;

	EFriendInvitePolicy LocalExpectedInvitePolicy = EFriendInvitePolicy::Private;
	EFriendInvitePolicy LocalNewInvitesPolicyValue = EFriendInvitePolicy::Private;
	EFriendInvitePolicy LocalDefaultInvitesPolicyValue = EFriendInvitePolicy::Public;
	
	FString LocalInvitesPolicyName = TEXT("ACCEPTINVITES");
	FString LocalStringNewInvitesPolicyValue = TEXT("PRIVATE");
	FString LocalStringDefaultInvitesPolicyValue = TEXT("PUBLIC");

	FFriendSettings LocalExpecetdSettings;
	LocalExpecetdSettings.SetSettingValue(LocalInvitesPolicyName, LocalStringNewInvitesPolicyValue);
	LocalExpecetdSettings.SettingsMap[LocalInvitesPolicyName] = LocalStringNewInvitesPolicyValue;

	FFriendSettings LocalDefaultSettings;
	LocalDefaultSettings.SetSettingValue(LocalInvitesPolicyName, LocalStringDefaultInvitesPolicyValue);
	LocalDefaultSettings.SettingsMap[LocalInvitesPolicyName] = LocalStringDefaultInvitesPolicyValue;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FFriendsEnsureFriendshipStep>(LocalUserNum, TargetUserNum, &LocalUserId, &TargetUserId, LocalListName, LocalIsFriendsListPopulated)
		.EmplaceStep<FFriendsUpdateFriendInvitePolicySettingsStep>(&LocalUserId, LocalNewInvitesPolicyValue, bLocalInAffectsExistingInvites)
		.EmplaceStep<FFriendsReadFriendsListStep>(LocalUserNum, LocalListName)
		.EmplaceStep<FFriendsQueryFriendSettingsStep>(&LocalUserId, LocalExpecetdSettings)
		.EmplaceStep<FFriendsGetFriendInvitePolicyStep>(&LocalUserId, LocalExpectedInvitePolicy)
		.EmplaceStep<FFriendsUpdateFriendInvitePolicySettingsStep>(&LocalUserId, LocalDefaultInvitesPolicyValue, bLocalInAffectsExistingInvites);

	RunToCompletion();
}
