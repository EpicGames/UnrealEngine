// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "Helpers/Identity/IdentityAutoLoginHelper.h"
#include "Helpers/Identity/IdentityLogoutHelper.h"

#include "Misc/CommandLine.h"
#include "OnlineSubsystemCatchHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_AUTOLOGIN_TAG IDENTITY_TAG "[autologin]"
#define EG_IDENTITY_AUTOLOGIN_LEGACY_EOS_FLOW_TAG IDENTITY_TAG "[autologin][.EOS]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity AutoLogin with valid inputs and using legacy login flow returns the expected result(Success Case)", EG_IDENTITY_AUTOLOGIN_LEGACY_EOS_FLOW_TAG)
{
	FTestDriver LocalDriver;
	int32 LocalUserNum = 0;
	int32 NumUsers = 1;

	TArray<FOnlineAccountCredentials> AccountCreds = GetCredentials(LocalUserNum, NumUsers);

	FString LoginCredentialsType = TEXT("AUTH_TYPE=") + AccountCreds[0].Type + ",";
	FString LoginCredentialsId = TEXT("AUTH_LOGIN=") + AccountCreds[0].Id + +",";
	FString LoginCredentialsPassword = TEXT("AUTH_PASSWORD=") + AccountCreds[0].Token;

	FCommandLine::Set(*(LoginCredentialsType + LoginCredentialsId + LoginCredentialsPassword));

	FTestPipeline LocalPipeline = LocalDriver.MakePipeline()
		.EmplaceStep<FIdentityAutoLoginStep>(LocalUserNum)
		.EmplaceStep<FIdentityLogoutStep>(LocalUserNum);

	FPipelineTestContext TestContext = FPipelineTestContext(GetSubsystem());
	REQUIRE(LocalDriver.AddPipeline(MoveTemp(LocalPipeline), TestContext));
	LocalDriver.RunToCompletion();
}

IDENTITY_TEST_CASE("Verify calling Identity AutoLogin with valid inputs returns the expected result(Success Case)", EG_IDENTITY_AUTOLOGIN_TAG)
{ 
	FTestDriver LocalDriver;
	int32 LocalUserNum = 0;
	int32 NumUsers = 1;
	
	TArray<FOnlineAccountCredentials> AccountCreds = GetCredentials(LocalUserNum, NumUsers);

	FString LoginCredentialsType = TEXT("AUTH_TYPE=") + AccountCreds[0].Type + ",";
	FString LoginCredentialsId = TEXT("AUTH_LOGIN=") + AccountCreds[0].Id + + ",";
	FString LoginCredentialsPassword = TEXT("AUTH_PASSWORD=") + AccountCreds[0].Token;

	FCommandLine::Set(*(LoginCredentialsType + LoginCredentialsId + LoginCredentialsPassword));

	FTestPipeline LocalPipeline = LocalDriver.MakePipeline()
		.EmplaceStep<FIdentityAutoLoginStep>(LocalUserNum)
		.EmplaceStep<FIdentityLogoutStep>(LocalUserNum);

	FPipelineTestContext TestContext = FPipelineTestContext(GetSubsystem());
	REQUIRE(LocalDriver.AddPipeline(MoveTemp(LocalPipeline), TestContext));
	LocalDriver.RunToCompletion();
}
