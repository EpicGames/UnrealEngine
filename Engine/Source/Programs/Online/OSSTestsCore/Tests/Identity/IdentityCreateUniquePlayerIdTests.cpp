// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityCreateUniquePlayerIdFromBinaryDataHelper.h"
#include "Helpers/Identity/IdentityCreateUniquePlayerIdFromStringHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_CREATEUNIQUEPLAYERID_TAG IDENTITY_TAG "[createuniqueplayerid]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity CreateUniquePlayerId from Binary Data with valid inputs returns the expected result(Success Case)", EG_IDENTITY_CREATEUNIQUEPLAYERID_TAG)
{
	FString PlayerGUIDString;
	FGuid PlayerGUID;
	FPlatformMisc::CreateGuid(PlayerGUID);
	PlayerGUIDString = PlayerGUID.ToString();
	int32 NumUsersToImplicitLogin = 0;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityCreateUniquePlayerIdFromBinaryDataStep>((uint8*)PlayerGUIDString.GetCharArray().GetData(), PlayerGUIDString.Len());

	RunToCompletion();
}

IDENTITY_TEST_CASE("Verify calling Identity CreateUniquePlayerId from String with valid inputs returns the expected result(Success Case)", EG_IDENTITY_CREATEUNIQUEPLAYERID_TAG)
{
	FString TestString = FString(TEXT("test_string"));
	int32 NumUsersToImplicitLogin = 0;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityCreateUniquePlayerIdFromStringStep>(TestString);

	RunToCompletion();
}
