// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Identity/IdentityGetAllUserAccountsHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_GETALLUSERACCOUNTS_TAG IDENTITY_TAG "[getalluseraccounts]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity GetAllUserAccounts with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETALLUSERACCOUNTS_TAG)
{
	int32 LocalUserNum = 0;
	int32 TargetUserNum = 1;
	FUniqueNetIdPtr LocalUniqueNetId = nullptr;
	FUniqueNetIdPtr TargetUniqueNetId = nullptr;
	int32 NumUsersToImplicitLogin = 2;
		
	TArray<FUniqueNetIdPtr> UserUniqueNetIds;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&UserUniqueNetIds](FUniqueNetIdPtr InUserId) { UserUniqueNetIds.Add(InUserId); })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&UserUniqueNetIds](FUniqueNetIdPtr InUserId) { UserUniqueNetIds.Add(InUserId); })
		.EmplaceStep<FIdentityGetAllUserAccountsStep>(&UserUniqueNetIds);

	RunToCompletion();
}
