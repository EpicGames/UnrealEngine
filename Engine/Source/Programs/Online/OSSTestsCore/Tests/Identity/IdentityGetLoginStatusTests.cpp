// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityLoginHelper.h"
#include "Helpers/Identity/IdentityLogoutHelper.h"
#include "Helpers/Identity/IdentityGetLoginStatusByLocalUserNumHelper.h"
#include "Helpers/Identity/IdentityGetLoginStatusByUserIdHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_GETLOGINSTATUS_TAG IDENTITY_TAG "[getloginstatus]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity GetLoginStatus by UserId with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETLOGINSTATUS_TAG)
{
	FTestDriver LocalDriver;
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	ELoginStatus::Type ExpectedLoginStatus = ELoginStatus::Type::LoggedIn;
	ELoginStatus::Type ExpectedLogoutStatus = ELoginStatus::Type::NotLoggedIn;
	int32 NumUsersToImplicitLogin = 0;
	int32 NumUsers = 1;
	TArray<FOnlineAccountCredentials> AccountCreds = GetCredentials(LocalUserNum, NumUsers);

	FTestPipeline LocalPipeline = LocalDriver.MakePipeline()
		.EmplaceStep<FIdentityLoginStep>(LocalUserNum, AccountCreds[0])
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetLoginStatusByUserIdStep>(&LocalUserId, ExpectedLoginStatus)
		.EmplaceStep<FIdentityLogoutStep>(LocalUserNum)
		.EmplaceStep<FIdentityGetLoginStatusByUserIdStep>(&LocalUserId, ExpectedLogoutStatus);

	REQUIRE(LocalDriver.AddPipeline(MoveTemp(LocalPipeline), GetSubsystem()));
	LocalDriver.RunToCompletion();
}

IDENTITY_TEST_CASE("Verify calling Identity GetLoginStatus by LocalUserNum with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETLOGINSTATUS_TAG)
{
	FTestDriver LocalDriver;
	int32 LocalUserNum = 0;
	ELoginStatus::Type ExpectedLoginStatus = ELoginStatus::Type::LoggedIn;
	ELoginStatus::Type ExpectedLogoutStatus = ELoginStatus::Type::NotLoggedIn;
	int32 NumUsersToImplicitLogin = 0;
	int32 NumUsers = 1;
	TArray<FOnlineAccountCredentials> AccountCreds = GetCredentials(LocalUserNum, NumUsers);

	FTestPipeline LocalPipeline = LocalDriver.MakePipeline()
		.EmplaceStep<FIdentityLoginStep>(LocalUserNum, AccountCreds[0])
		.EmplaceStep<FIdentityGetLoginStatusByLocalUserNumStep>(LocalUserNum, ExpectedLoginStatus)
		.EmplaceStep<FIdentityLogoutStep>(LocalUserNum)
		.EmplaceStep<FIdentityGetLoginStatusByLocalUserNumStep>(LocalUserNum, ExpectedLogoutStatus);

	REQUIRE(LocalDriver.AddPipeline(MoveTemp(LocalPipeline), GetSubsystem()));
	LocalDriver.RunToCompletion();
}
