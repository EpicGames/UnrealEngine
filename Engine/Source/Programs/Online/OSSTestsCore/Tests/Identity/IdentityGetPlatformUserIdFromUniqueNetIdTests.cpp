// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Identity/IdentityGetPlatformUserIdFromUniqueNetIdHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_GETPLATFORMUSERID_TAG IDENTITY_TAG "[getplatformuserid]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity GetPlatformUserId from UniqueNetId with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETPLATFORMUSERID_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUniqueNetId = nullptr;
	int32 NumUsersToImplicitLogin = 1;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUniqueNetId](FUniqueNetIdPtr InUserId) { LocalUniqueNetId = InUserId; })
		.EmplaceStep<FIdentityGetPlatformUserIdFromUniqueNetIdStep>(&LocalUniqueNetId);

	RunToCompletion();
}
