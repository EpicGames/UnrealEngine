// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Identity/IdentityGetPlayerNicknameByUserIdHelper.h"
#include "Helpers/Identity/IdentityGetPlayerNicknameByLocalUserNumHelper.h"

#if UE_ENABLE_ICU
#include "Internationalization/BreakIterator.h"
#endif

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_GETPLAYERNICKNAME_TAG IDENTITY_TAG "[getplayernickname]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity GetPlayerNickname by UserId with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETPLAYERNICKNAME_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 NumUsersToImplicitLogin = 1;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetPlayerNicknameByUserIdStep>(&LocalUserId);

	RunToCompletion();
}

IDENTITY_TEST_CASE("Verify calling Identity GetPlayerNickname by LocalUserNum with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETPLAYERNICKNAME_TAG)
{
	int32 LocalUserNum = 1;
	int32 NumUsersToImplicitLogin = 1;
	
	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetPlayerNicknameByLocalUserNumStep>(LocalUserNum);
	
	RunToCompletion();
}

#if UE_ENABLE_ICU
IDENTITY_TEST_CASE("Verify that player nick name length counting works well", EG_IDENTITY_GETPLAYERNICKNAME_TAG)
{
	REQUIRE(FInternationalization::Get().IsInitialized());

	// Café
	FString StringToCheck(TEXTVIEW("\u0043\u0061\u0066\u0065\u0301"));

	TSharedRef<IBreakIterator> GraphemeBreakIterator = FBreakIterator::CreateCharacterBoundaryIterator();
	GraphemeBreakIterator->SetString(StringToCheck);
	GraphemeBreakIterator->ResetToBeginning();

	int32 Count = 0;
	for (int32 CurrentCharIndex = GraphemeBreakIterator->MoveToNext(); CurrentCharIndex != INDEX_NONE; CurrentCharIndex = GraphemeBreakIterator->MoveToNext())
	{
		Count++;
	}

	// Although there are 5 code points, the number of graphemes is 4
	CHECK(StringToCheck.Len() == 5);
	CHECK(Count == 4);
}
#endif
