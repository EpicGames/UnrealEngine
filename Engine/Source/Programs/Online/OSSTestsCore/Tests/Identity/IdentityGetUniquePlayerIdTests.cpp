// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_GETUNIQUEPLAYERID_TAG IDENTITY_TAG "[getuniqueplayerid]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity GetUniquePlayerId with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETUNIQUEPLAYERID_TAG)
{
	int32 LocalUserNum = 0;
	int32 NumUsersToImplicitLogin = 1;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum);
	
	RunToCompletion();
}
