// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Identity/IdentityGetUserAccountHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_GETUSERACCOUNT_TAG IDENTITY_TAG "[getuseraccount]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity GetUserAccount with valid inputs returns the expected result(Success Case)", EG_IDENTITY_GETUSERACCOUNT_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUniqueNetId = nullptr;
	int32 NumUsersToImplicitLogin = 1;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUniqueNetId](FUniqueNetIdPtr InUserId) { LocalUniqueNetId = InUserId; })
		.EmplaceStep<FIdentityGetUserAccountStep>(&LocalUniqueNetId);

	RunToCompletion();
}
