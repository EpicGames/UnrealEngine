// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_LOGIN_TAG IDENTITY_TAG "[login]"
#define EG_IDENTITY_LOGIN_LEGACY_FLOWEOS_TAG IDENTITY_TAG "[login][.EOS]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity Login with valid inputs and using legacy login flow returns the expected result(Success Case)", EG_IDENTITY_LOGIN_LEGACY_FLOWEOS_TAG)
{
	int32 NumUsersToImplicitLogin = 2;

	GetLoginPipeline(NumUsersToImplicitLogin);

	RunToCompletion();
}

IDENTITY_TEST_CASE("Verify calling Identity Login with valid inputs returns the expected result(Success Case)", EG_IDENTITY_LOGIN_TAG)
{
	int32 NumUsersToImplicitLogin = 2;

	GetLoginPipeline(NumUsersToImplicitLogin);

	RunToCompletion();
}