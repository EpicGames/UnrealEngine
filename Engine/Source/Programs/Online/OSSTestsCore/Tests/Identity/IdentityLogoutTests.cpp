// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"
#include "Helpers/Identity/IdentityLoginHelper.h"
#include "Helpers/Identity/IdentityLogoutHelper.h"

#define IDENTITY_TAG "[suite_identity]"
#define EG_IDENTITY_LOGOUT_TAG IDENTITY_TAG "[logout]"

#define IDENTITY_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, IDENTITY_TAG __VA_ARGS__)

IDENTITY_TEST_CASE("Verify calling Identity Logout with valid inputs returns the expected result(Success Case)", EG_IDENTITY_LOGOUT_TAG)
{
	int32 LocalUserNum = 0;
	int32 NumUsers = 1;
	TArray<FOnlineAccountCredentials> AccountCreds = GetCredentials(LocalUserNum, NumUsers);

	GetPipeline()
		.EmplaceStep<FIdentityLoginStep>(LocalUserNum, AccountCreds[0])
		.EmplaceStep<FIdentityLogoutStep>(LocalUserNum);

	RunToCompletion();
}
