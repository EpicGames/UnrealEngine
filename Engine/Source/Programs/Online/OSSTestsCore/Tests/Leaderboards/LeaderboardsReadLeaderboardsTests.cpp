// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Leaderboards/LeaderboardsWriteLeaderboardsHelper.h"
#include "Helpers/Leaderboards/LeaderboardsReadLeaderboardsHelper.h"

#define LEADERBOARDS_TAG "[suite_leaderboards]"
#define EG_LEADERBOARDS_READLEADERBOARDS_TAG LEADERBOARDS_TAG "[readleaderboards]"

#define EG_LEADERBOARDS_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, LEADERBOARDS_TAG __VA_ARGS__)

EG_LEADERBOARDS_TEST_CASE("Verify calling ReadLeaderboards with valid inputs returns the expected result(Success Case)", EG_LEADERBOARDS_READLEADERBOARDS_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;

	FName LocalSessionName = TEXT("FakeSessionName");
	const FString LocalNameForLeaderboard = TEXT("Name1");
	FOnlineLeaderboardWrite WriteObject;

	WriteObject.LeaderboardNames = { LocalNameForLeaderboard };
	WriteObject.SortMethod = ELeaderboardSort::Ascending;
	WriteObject.UpdateMethod = ELeaderboardUpdateMethod::KeepBest;
	WriteObject.RatedStat = FString(TEXT("Scores"));

	TArray<FUniqueNetIdRef> LocalPlayers;
	FOnlineLeaderboardReadRef LocalReadObject = MakeShared<FOnlineLeaderboardRead>();
	LocalReadObject.Get().LeaderboardName = { LocalNameForLeaderboard };
	int32 NumUsersToImplicitLogin = 1;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InPlayerId) {LocalUserId = InPlayerId; })
		.EmplaceStep<FLeaderboardsWriteLeaderboardsStep>(LocalSessionName, &LocalUserId, WriteObject)
		.EmplaceStep<FLeaderboardsReadLeaderboardsStep>(LocalPlayers, LocalReadObject, [&LocalUserId](TArray<FUniqueNetIdRef>& InPlayers) { InPlayers.Add(LocalUserId.ToSharedRef()); });

    RunToCompletion();
}
