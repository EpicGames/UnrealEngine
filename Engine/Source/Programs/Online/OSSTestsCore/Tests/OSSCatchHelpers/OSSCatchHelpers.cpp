// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineSubsystemCatchHelper.h"

#define SELFTEST_TAG "[selftest][osscatchhelper]"
#define GENERATETAGS_TAG "[generatetags]"
#define CHECKALLTAGSISIN_TAG "[checkalltagsisin]"
#define SHOULDDISABLETEST_TAG "[shoulddisabletest]"
#define SELFTEST_TEST_CASE(x, ...) TEST_CASE(x, SELFTEST_TAG __VA_ARGS__)

SELFTEST_TEST_CASE("GenerateTags append MayFailTags case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append multiple match MayFailTags case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[bar]", "[foo]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match MayFailTags case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[wiz]", "[foo]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo],bar" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag no match case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo],[wiz]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags don't append MayFailTags no match case", GENERATETAGS_TAG)
{
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[wiz]" };
	FString TestTags = "[foo][bar]";
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags append ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[foo]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append multiple match ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[bar]", "[foo]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[wiz]", "[foo]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags don't append ShouldFail no match case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[wiz]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags appends MayFail and ShouldFail Case", GENERATETAGS_TAG)
{
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo]" };
	SkipTags.ShouldFailTags = { "[bar]" };
	FString TestTags = "[foo][bar]";
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail][!shouldfail]"));
}


SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[foo],bar" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag no match ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[foo],[wiz]" };
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags appends no tags case", GENERATETAGS_TAG)
{
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[wiz]" };
	SkipTags.ShouldFailTags = {  };
	FString TestTags = "[foo][bar]";
	FString OutTags = FOnlineSubsystemFixtureInvoker::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("ShouldDisableTest returns true on single-tag config", SHOULDDISABLETEST_TAG)
{
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.DisableTestTags = { "[foo]" };
	FString TestTags = "[foo][bar]";
	CHECK(FOnlineSubsystemFixtureInvoker::ShouldDisableTest("TestService", SkipTags, TestTags) == FOnlineSubsystemFixtureInvoker::EDisableReason::DisableTagPresence);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns true on multi-tag config", SHOULDDISABLETEST_TAG)
{
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.DisableTestTags = { "[foo],bar" };
	FString TestTags = "[foo][bar]";
	CHECK(FOnlineSubsystemFixtureInvoker::ShouldDisableTest("TestService", SkipTags, TestTags) == FOnlineSubsystemFixtureInvoker::EDisableReason::DisableTagPresence);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns true on !<service>", SHOULDDISABLETEST_TAG)
{
	FString TestTags = "[foo][bar][!TestService]";
	CHECK(FOnlineSubsystemFixtureInvoker::ShouldDisableTest("TestService", {}, TestTags) == FOnlineSubsystemFixtureInvoker::EDisableReason::AgainstService);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns false with no tags and no config skips", SHOULDDISABLETEST_TAG)
{
	FString TestTags = "[foo][bar]";
	CHECK(FOnlineSubsystemFixtureInvoker::ShouldDisableTest("TestService", {}, TestTags) == FOnlineSubsystemFixtureInvoker::EDisableReason::Success);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns false with no matching no-tags and no matching multi-tag config skips", SHOULDDISABLETEST_TAG)
{
	FOnlineSubsystemFixtureInvoker::FReportingSkippableTags SkipTags;
	SkipTags.DisableTestTags = { "[foo],wiz" };
	FString TestTags = "[foo][bar]";
	CHECK(FOnlineSubsystemFixtureInvoker::ShouldDisableTest("TestService", SkipTags, TestTags) == FOnlineSubsystemFixtureInvoker::EDisableReason::Success);
}

SELFTEST_TEST_CASE("CheckAllTagsIsIn(TArray, FString) true cases", CHECKALLTAGSISIN_TAG)
{
	TArray<FString> TestTags = { "bob", "alice", "foo" };
	CAPTURE(TestTags);

	// Truthy Cases
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "bob, alice") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "bob,alice") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, " bob,alice ") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "foo") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, ",foo") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "bob,alice,foo") == true);

	//Bracket Parsing
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "[bob],[alice],[foo]") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "[bob], [alice,foo]") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "bob],  alice],  [foo]  ,") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, ",[foo]") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, ",foo]") == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "[wiz]") == false);

	// Negative Cases
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "bob,alice,foo,wiz") == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "bob,wiz") == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, ",wiz") == false);

	// Bound Checks
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, ",") == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, "") == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn({}, "") == false);
}


SELFTEST_TEST_CASE("CheckAllTagsIsIn(TArray, TArray) true cases", CHECKALLTAGSISIN_TAG)
{
	TArray<FString> TestTags = { "bob", "alice", "foo" };
	CAPTURE(TestTags);

	// Truthy Cases
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, { "bob", "alice" }) == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, { "bob", "alice", "foo" }) == true);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, TArray<FString>({ "foo" })) == true);

	// Negative Cases
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, TArray<FString>({ "wiz" })) == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, { "bob", "alice", "foo", "wiz" }) == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, { "bob", "alice", "wiz" }) == false);

	// Bounds
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn(TestTags, TArray<FString>({})) == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn({}, TArray<FString>({})) == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn({}, TArray<FString>({ "wiz" })) == false);
	CHECK(FOnlineSubsystemFixtureInvoker::CheckAllTagsIsIn({ "wiz" }, TArray<FString>({})) == false);
}