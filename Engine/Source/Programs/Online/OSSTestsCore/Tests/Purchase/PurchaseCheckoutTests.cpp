// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"
#include "OnlineSubsystemCatchHelper.h"
#include "Helpers/Identity/IdentityLoginHelper.h"
#include "Helpers/Identity/IdentityLogoutHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Purchase/PurchaseCheckoutHelper.h"

#define PURCHASE_TAG "[suite_purchase]"
#define EG_PURCHASE_CHECKOUT_TAG PURCHASE_TAG "[checkout]"

#define PURCHASE_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, PURCHASE_TAG __VA_ARGS__)

PURCHASE_TEST_CASE("Verify calling Checkout with valid inputs returns the expected result(Success Case)", EG_PURCHASE_CHECKOUT_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	int32 LocalQuantity = 1;
	bool bLocalIsConsumable = true;
	const FOfferNamespace LocalOfferNamespace = TEXT("");
	const FUniqueOfferId& LocalOfferId = TEXT("Item1_Id");
	const FString LocalItemName = TEXT("Cool Item1");
	FPurchaseCheckoutRequest PurchCheckRequest;
	PurchCheckRequest.AddPurchaseOffer(LocalOfferNamespace, LocalOfferId, LocalQuantity, bLocalIsConsumable);
	int32 NumUsersToImplicitLogin = 1;			

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FPurchaseCheckoutStep>(&LocalUserId, &PurchCheckRequest, LocalOfferId, LocalItemName);
	
	RunToCompletion();
}