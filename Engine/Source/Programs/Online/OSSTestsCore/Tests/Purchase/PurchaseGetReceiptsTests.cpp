// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"
#include "Logging/LogScopedVerbosityOverride.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Purchase/PurchaseGetReceiptsHelper.h"
#include "Helpers/Purchase/PurchaseQueryReceiptsHelper.h"
#include "Helpers/Purchase/PurchaseCheckoutHelper.h"

#define PURCHASE_TAG "[suite_purchase]"
#define EG_PURCHASE_GETRECEIPTS_TAG PURCHASE_TAG "[getreceipts]"

#define PURCHASE_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, PURCHASE_TAG __VA_ARGS__)

PURCHASE_TEST_CASE("Verify calling GetReceipts with valid inputs returns the expected result(Success Case)", EG_PURCHASE_GETRECEIPTS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnline, ELogVerbosity::Verbose);

	const int32 Quantity = 1;
	FString ItemName;
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	bool bRestoreReceipts = false;
	bool bIsShouldBeEmpty = false;

	TArray<FPurchaseReceipt>* PurchaseReceipts = nullptr;

	FPurchaseCheckoutRequest PurchCheckRequest = {};

	FTestPipeline& LoginPipeline = GetLoginPipeline({ LocalUserId });

	LoginPipeline
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; });

	if (GetSubsystem() == "EOS")
	{
		ItemName = TEXT("SessionMatchmaking-TestSearchable");
		LoginPipeline
			.EmplaceStep<FPurchaseQueryReceiptsStep>(&LocalUserId, bRestoreReceipts);
	}
	else if (GetSubsystem() == "NULL")
	{
		const FOfferNamespace OfferNamespace = TEXT("");
		const FUniqueOfferId& OfferId = TEXT("Item1_Id");
		ItemName = TEXT("Cool Item1");
		bool bIsConsumable = true;
		PurchCheckRequest.AddPurchaseOffer(OfferNamespace, OfferId, Quantity, bIsConsumable);

		LoginPipeline
			.EmplaceStep<FPurchaseCheckoutStep>(&LocalUserId, &PurchCheckRequest, OfferId, ItemName);
	}

	LoginPipeline
		.EmplaceStep<FPurchaseGetReceiptsStep>(&LocalUserId, [&PurchaseReceipts](TArray<FPurchaseReceipt>* InPurchaseReceipts) {PurchaseReceipts = InPurchaseReceipts; })
		.EmplaceLambda([&PurchaseReceipts, bIsShouldBeEmpty, ItemName, Quantity](IOnlineSubsystem* OnlineSubsystem)
			{
				CHECK(PurchaseReceipts->IsEmpty() == bIsShouldBeEmpty);
				CHECK(PurchaseReceipts->GetData()->ReceiptOffers.Num() == Quantity);
				CHECK(PurchaseReceipts->GetData()->ReceiptOffers[0].LineItems[0].ItemName == ItemName);
			});

	RunToCompletion();
}


