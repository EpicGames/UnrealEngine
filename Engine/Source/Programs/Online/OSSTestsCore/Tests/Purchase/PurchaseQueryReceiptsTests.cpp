// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"
#include "Logging/LogScopedVerbosityOverride.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Purchase/PurchaseQueryReceiptsHelper.h"
#include "Helpers/Purchase/PurchaseGetReceiptsHelper.h"
#include "Helpers/Purchase/PurchaseCheckoutHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"

#include "Helpers/Identity/IdentityLogoutHelper.h"

#define PURCHASE_TAG "[suite_purchase]"
#define EG_PURCHASE_QUERYRECEIPTS_TAG PURCHASE_TAG "[queryreceipts]"

#define PURCHASE_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, PURCHASE_TAG __VA_ARGS__)

PURCHASE_TEST_CASE("Basic compile test")
{
	TSharedPtr<IOnlinePurchase> Commerce;
	CHECK(!Commerce.IsValid());
}

PURCHASE_TEST_CASE("Verify that QueryReceipts caches an empty list if there are no offers", EG_PURCHASE_QUERYRECEIPTS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnline, ELogVerbosity::Verbose);

	const FUniqueOfferId& OfferId = TEXT("Item1_Id");

	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	bool bRestoreReceipts = false;
	bool bIsShouldBeEmpty = true;

	int32 NumUsersToImplicitLogin = 1;
	int32 TestAccountIndex = 1;
	TArray<FPurchaseReceipt>* PurchaseReceipts = nullptr;

	GetLoginPipeline(TestAccountIndex, { LocalUserId })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FPurchaseQueryReceiptsStep>(&LocalUserId, bRestoreReceipts, OfferId)
		.EmplaceStep<FPurchaseGetReceiptsStep>(&LocalUserId, [&PurchaseReceipts](TArray<FPurchaseReceipt>* InPurchaseReceipts) {PurchaseReceipts = InPurchaseReceipts; })
		.EmplaceLambda([&PurchaseReceipts, bIsShouldBeEmpty](const IOnlineSubsystem* OnlineSubsystem)
			{
				CHECK(PurchaseReceipts->IsEmpty() == bIsShouldBeEmpty);
			});

	RunToCompletion();
}

PURCHASE_TEST_CASE("Verify calling QueryReceipts with valid inputs returns the expected result and expected list of receipts(Success Case)", EG_PURCHASE_QUERYRECEIPTS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnline, ELogVerbosity::Verbose);

	const int32 Quantity = 1;
	FString ItemName;
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	bool bRestoreReceipts = false;
	bool bIsShouldBeEmpty = false;

	int32 NumUsersToImplicitLogin = 1;
	TArray<FPurchaseReceipt>* PurchaseReceipts = nullptr;

	FPurchaseCheckoutRequest PurchCheckRequest = {};

	FTestPipeline& LoginPipeline = GetLoginPipeline({ LocalUserId });

	LoginPipeline
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; });

	if (GetSubsystem() == "EOS")
	{
		ItemName = TEXT("SessionMatchmaking-TestSearchable");
		LoginPipeline
			.EmplaceStep<FPurchaseQueryReceiptsStep>(&LocalUserId, bRestoreReceipts);
	}
	else if (GetSubsystem() == "NULL")
	{
		const FOfferNamespace OfferNamespace = TEXT("");
		const FUniqueOfferId& OfferId = TEXT("Item1_Id");
		ItemName = TEXT("Cool Item1");
		bool bIsConsumable = true;
		PurchCheckRequest.AddPurchaseOffer(OfferNamespace, OfferId, Quantity, bIsConsumable);

		LoginPipeline
			.EmplaceStep<FPurchaseCheckoutStep>(&LocalUserId, &PurchCheckRequest, OfferId, ItemName)
			.EmplaceStep<FPurchaseQueryReceiptsStep>(&LocalUserId, bRestoreReceipts, OfferId);
	}

	LoginPipeline
		.EmplaceStep<FPurchaseGetReceiptsStep>(&LocalUserId, [&PurchaseReceipts](TArray<FPurchaseReceipt>* InPurchaseReceipts) {PurchaseReceipts = InPurchaseReceipts; })
		.EmplaceLambda([&PurchaseReceipts, bIsShouldBeEmpty, ItemName, Quantity](IOnlineSubsystem* OnlineSubsystem)
			{
				CHECK(PurchaseReceipts->IsEmpty() == bIsShouldBeEmpty);
				CHECK(PurchaseReceipts->GetData()->ReceiptOffers.Num() == Quantity);
				CHECK(PurchaseReceipts->GetData()->ReceiptOffers[0].LineItems[0].ItemName == ItemName);
			});

	RunToCompletion();
}
