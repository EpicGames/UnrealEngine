// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Purchase/PurchaseQueryReceiptsHelper.h"
#include "Helpers/Purchase/PurchaseGetReceiptsHelper.h"
#include "Helpers/Purchase/PurchaseCheckoutHelper.h"
#include "Helpers/Purchase/PurchaseRedeemCodeHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"

#define PURCHASE_TAG "[suite_purchase]"
#define EG_PURCHASE_REDEEMCODE_TAG PURCHASE_TAG "[redeemcode]"

#define PURCHASE_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, PURCHASE_TAG __VA_ARGS__)

PURCHASE_TEST_CASE("Verify calling RedeemCode with valid inputs returns the expected result(Success Case)", EG_PURCHASE_REDEEMCODE_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr LocalUserId = nullptr;
	const FUniqueOfferId& LocalOfferId = TEXT("Item1_Id");
	const FString LocalItemName = TEXT("Cool Item1");
	int32 LocalQuantity = 1;
	bool bLocalIsConsumable = true;
	const FOfferNamespace LocalOfferNamespace = TEXT("");
	FPurchaseCheckoutRequest PurchCheckRequest = {};
	PurchCheckRequest.AddPurchaseOffer(LocalOfferNamespace, LocalOfferId, LocalQuantity, bLocalIsConsumable);
	FRedeemCodeRequest InRedeemCodeRequest;
	InRedeemCodeRequest.Code = LocalOfferId;
	int32 NumUsersToImplicitLogin = 1;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FPurchaseRedeemCodeStep>(&LocalUserId, InRedeemCodeRequest, LocalOfferId, LocalOfferId);
	
	RunToCompletion();
}