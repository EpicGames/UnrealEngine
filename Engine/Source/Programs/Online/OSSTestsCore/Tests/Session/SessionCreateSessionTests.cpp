// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_CREATESESSION_TAG SESSION_TAG "[createsession]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session CreateSession by HostingPlayerId with valid inputs returns the expected result(Success Case)", EG_SESSION_CREATESESSION_TAG)
{
	int32 LocalUserNum = 0;
	FUniqueNetIdPtr UserId = nullptr;
	int32 PublicConnections = 1;
	int32 NumUsersToImplicitLogin = 1;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&UserId](FUniqueNetIdPtr InUserId) {UserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&UserId, SessionName, SessionSettings)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}

SESSION_TEST_CASE("Verify calling Session CreateSession by HostingPlayerNum with valid inputs returns the expected result(Success Case)", EG_SESSION_CREATESESSION_TAG)
{
	int32 LocalUserNum = 0;
	int32 PublicConnections = 1;
	int32 NumUsersToImplicitLogin = 1;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);
	
	FOnlineSessionSettings SessionSettings;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FSessionCreateSessionStep>(LocalUserNum, SessionName, SessionSettings)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);
	
	RunToCompletion();
}
