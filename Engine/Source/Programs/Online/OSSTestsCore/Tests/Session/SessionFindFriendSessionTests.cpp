// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Friends/FriendsEnsureFriendshipHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionFindFriendSessionHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_FINDFRIENDSESSION_TAG SESSION_TAG "[findfriendsession]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session FindFriendSession with LocalUserId and valid inputs returns the expected result(Success Case)", EG_SESSION_FINDFRIENDSESSION_TAG)
{
	int32 LocalUserNum = 0;
	int32 TargetUserNum = 1;
	int32 PublicConnections = 2;
	int32 PrivateConnections = 1;
	int32 NumUsersToImplicitLogin = 2;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineServiceAndPing);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.bShouldAdvertise = true;
	SessionSettings.bAllowJoinInProgress = true;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.NumPrivateConnections = PrivateConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);

	FString LocalListName = EFriendsLists::ToString(EFriendsLists::Default);
	bool bLocalIsFriendsListPopulated = true;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FFriendsEnsureFriendshipStep>(LocalUserNum, TargetUserNum, &LocalUserId, &TargetUserId, LocalListName, bLocalIsFriendsListPopulated)
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings)
		.EmplaceStep<FSessionFindFriendSessionStep>(&LocalUserId, LocalUserNum, &TargetUserId)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}