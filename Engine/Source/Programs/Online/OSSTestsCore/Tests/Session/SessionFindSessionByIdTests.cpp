// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionFindSessionByIdHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_FINDSESSIONBYID_TAG SESSION_TAG "[findsessionbyid]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session FindSessionById with SessionKey and valid inputs returns the expected result(Success case)", EG_SESSION_FINDSESSIONBYID_TAG)
{
	int32 LocalUserNum = 0;
	int32 TargetUserNum = 1;
	int32 PublicConnections = 2;
	int32 NumBots = 3;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FString SessionKey = TEXT("1234session");
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);
	FOnlineSessionSetting NumbotsSetting(NumBots, EOnlineDataAdvertisementType::ViaOnlineService);
	FOnlineSessionSetting SessionkeySetting(SessionKey, EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.bShouldAdvertise = true;
	SessionSettings.bAllowJoinInProgress = true;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);
	SessionSettings.Settings.Add(SETTING_NUMBOTS, NumbotsSetting);
	SessionSettings.Settings.Add(SETTING_SESSIONKEY, SessionkeySetting);

	int32 NumUsersToImplicitLogin = 2;

	TSharedPtr<FNamedOnlineSession> NamedOnlineSession = nullptr;
	
	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings, [&NamedOnlineSession](TSharedPtr<FNamedOnlineSession> InNamedOnlineSession){ NamedOnlineSession = MoveTemp(InNamedOnlineSession); })
		.EmplaceStep<FSessionFindSessionByIdStep>(&LocalUserId, &TargetUserId, &NamedOnlineSession, SessionKey)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}

SESSION_TEST_CASE("Verify calling Session FindSessionById without SessionKey and with valid inputs returns the expected result(Success case)", EG_SESSION_FINDSESSIONBYID_TAG)
{
	int32 LocalUserNum = 0;
	int32 TargetUserNum = 1;
	int32 PublicConnections = 2;
	int32 PrivateConnections = 1;
	int32 NumBots = 3;
	int32 NumUsersToImplicitLogin = 2;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);
	FOnlineSessionSetting NumbotsSetting(NumBots, EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.bShouldAdvertise = true;
	SessionSettings.bAllowJoinInProgress = true;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.NumPrivateConnections = PrivateConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);
	SessionSettings.Settings.Add(SETTING_NUMBOTS, NumbotsSetting);

	TSharedPtr<FNamedOnlineSession> NamedOnlineSession = nullptr;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings, [&NamedOnlineSession](TSharedPtr<FNamedOnlineSession> InNamedOnlineSession) { NamedOnlineSession = MoveTemp(InNamedOnlineSession); })
		.EmplaceStep<FSessionFindSessionByIdStep>(&LocalUserId, &TargetUserId, &NamedOnlineSession)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}