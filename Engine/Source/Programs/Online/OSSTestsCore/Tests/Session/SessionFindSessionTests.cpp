// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionFindSessionHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_FINDSESSION_TAG SESSION_TAG "[findsession]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling FindSessionby SearchingPlayerNum and SearchPlayerId valid inputs returns the expected result(Success case)", EG_SESSION_FINDSESSION_TAG)
{
	int32 LocalUserNum = 0;
	int32 PublicConnections = 1;
	int32 PrivateConnections = 1;
	int32 NumBots = 3;
	int32 NumUsersToImplicitLogin = 1;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);
	FOnlineSessionSetting NumbotsSetting(NumBots, EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.bShouldAdvertise = true;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.NumPrivateConnections = PrivateConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);
	SessionSettings.Settings.Add(SETTING_NUMBOTS, NumbotsSetting);

	FOnlineSessionSearch SearchSetting;
	SearchSetting.QuerySettings.Set(SETTING_NUMBOTS, NumBots, EOnlineComparisonOp::Equals);
	
	TSharedRef<FOnlineSessionSearch> SearchSettingRef = MakeShared<FOnlineSessionSearch>(SearchSetting);

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings)
		.EmplaceStep<FSessionFindSessionStep>(LocalUserNum, SearchSettingRef)
		.EmplaceStep<FSessionFindSessionStep>(&LocalUserId, SearchSettingRef)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}
