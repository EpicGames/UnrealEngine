// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"
#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionFindFriendSessionHelper.h"
#include "Helpers/Session/SessionGetResolvedConnectStringHelper.h"
#include "Helpers/Session/SessionFindSessionByIdHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_GETRESOLVEDCONNECTSTRING_TAG SESSION_TAG "[getresolvedconnectstring]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session GetConnectResolvedConnectString with valid inputs returns the expected result(Success Case)", EG_SESSION_GETRESOLVEDCONNECTSTRING_TAG)
{
	int32 LocalUserNum = 0;
	int32 TargetUserNum = 1;
	int32 PublicConnections = 2;
	int32 NumUsersToImplicitLogin = 2;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.bShouldAdvertise = true;
	SessionSettings.bAllowJoinViaPresence = true;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);

	TSharedPtr<FNamedOnlineSession> NamedOnlineSession = nullptr;
	TSharedPtr<FOnlineSessionSearchResult> SearchResult = nullptr;

	EOnJoinSessionCompleteResult::Type ExpectedSessionSuccessType = EOnJoinSessionCompleteResult::Type::Success;
	EOnJoinSessionCompleteResult::Type ExpectedSessionUnknownErrorType = EOnJoinSessionCompleteResult::Type::UnknownError;

	FName PortType = NAME_BeaconPort;
	FString ConnectInfo = TEXT("");

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings, [&NamedOnlineSession](TSharedPtr<FNamedOnlineSession> InNamedOnlineSession) { NamedOnlineSession = MoveTemp(InNamedOnlineSession); })
		.EmplaceStep<FSessionFindSessionByIdStep>(&LocalUserId, &TargetUserId, &NamedOnlineSession, [&SearchResult](TSharedPtr<FOnlineSessionSearchResult> InSearchResult) { SearchResult = MoveTemp(InSearchResult); })
		.EmplaceStep<FSessionGetResolvedConnectStringStep>(&SearchResult, PortType, ConnectInfo)
		.EmplaceStep<FSessionGetResolvedConnectStringStep>(SessionName, ConnectInfo, PortType)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}

