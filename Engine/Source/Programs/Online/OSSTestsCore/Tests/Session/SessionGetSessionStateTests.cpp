// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionStartSessionHelper.h"
#include "Helpers/Session/SessionEndSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionGetSessionStateHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_GETSESSIONSTATE_TAG SESSION_TAG "[getsessionstate]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session GetSessionState with valid inputs returns the expected result(Success Case)", EG_SESSION_GETSESSIONSTATE_TAG)
{
	int32 LocalUserNum = 0;
	int32 PublicConnections = 1;
	int32 NumUsersToImplicitLogin = 1;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);
	
	const EOnlineSessionState::Type ExpectedNoSessionType = EOnlineSessionState::Type::NoSession;
	const EOnlineSessionState::Type ExpectedPendingType = EOnlineSessionState::Type::Pending;
	const EOnlineSessionState::Type ExpectedInProgressType = EOnlineSessionState::Type::InProgress;
	const EOnlineSessionState::Type ExpectedEndedType = EOnlineSessionState::Type::Ended;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FSessionGetSessionStateStep>(SessionName, ExpectedNoSessionType)
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings)
		.EmplaceStep<FSessionGetSessionStateStep>(SessionName, ExpectedPendingType)
		.EmplaceStep<FSessionStartSessionStep>(SessionName)
		.EmplaceStep<FSessionGetSessionStateStep>(SessionName, ExpectedInProgressType)
		.EmplaceStep<FSessionEndSessionStep>(SessionName)
		.EmplaceStep<FSessionGetSessionStateStep>(SessionName, ExpectedEndedType)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName)
		.EmplaceStep<FSessionGetSessionStateStep>(SessionName, ExpectedNoSessionType);

	RunToCompletion();
}
