// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityCreateUniquePlayerIdFromStringHelper.h"
#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionRegisterPlayerHelper.h"
#include "Helpers/Session/SessionRegisterPlayersHelper.h"
#include "Helpers/Session/SessionUnregisterPlayerHelper.h"
#include "Helpers/Session/SessionUnregisterPlayersHelper.h"
#include "Helpers/Session/SessionIsPlayerInSessionHelper.h"

#define SESSION_TAG "[suite_session]"
#define EG_SESSION_REGISTERPLAYERSSTEP_TAG SESSION_TAG "[registerplayer]"

#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session RegisterPlayer with valid inputs returns the expected result(Success Case)", EG_SESSION_REGISTERPLAYERSSTEP_TAG)
{
	int32 LocalUserNum = 0;
	int32 PublicConnections = 1;
	int32 PrivateConnections = 1;
	int32 NumUsersToImplicitLogin = 1;
	FUniqueNetIdPtr UserId = nullptr;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.NumPrivateConnections = PrivateConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);

	FString FirstPlayer = FString(TEXT("FirstPlayer"));
	bool bWasInvited = true;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&UserId](FUniqueNetIdPtr InUserId) {UserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&UserId, SessionName, SessionSettings)
		.EmplaceStep<FIdentityCreateUniquePlayerIdFromStringStep>(FirstPlayer, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FSessionRegisterPlayerStep>(SessionName, &TargetUserId, bWasInvited)
		.EmplaceStep<FSessionIsPlayerInSessionStep>(SessionName, &TargetUserId)
		.EmplaceStep<FSessionUnregisterPlayerStep>(SessionName, &TargetUserId)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}

SESSION_TEST_CASE("Verify calling Session RegisterPlayers with valid inputs returns the expected result(Success Case)", EG_SESSION_REGISTERPLAYERSSTEP_TAG)
{
	int32 LocalUserNum = 0;
	int32 PublicConnections = 1;
	int32 PrivateConnections = 2;
	int32 NumUsersToImplicitLogin = 1;
	FUniqueNetIdPtr UserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting Setting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);

	FOnlineSessionSettings SessionSettings;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.NumPrivateConnections = PrivateConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, Setting);

	FString FirstPlayer = FString(TEXT("FirstPlayer"));
	FString SecondPlayer = FString(TEXT("SecondPlayer"));

	TArray<FUniqueNetIdRef> Players;	
	bool bWasInvited = true;

	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&UserId](FUniqueNetIdPtr InUserId) {UserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&UserId, SessionName, SessionSettings)
		.EmplaceStep<FIdentityCreateUniquePlayerIdFromStringStep>(FirstPlayer, [&Players](FUniqueNetIdPtr InUserId) {Players.Emplace(InUserId.ToSharedRef()); })
		.EmplaceStep<FIdentityCreateUniquePlayerIdFromStringStep>(SecondPlayer, [&Players](FUniqueNetIdPtr InUserId) {Players.Emplace(InUserId.ToSharedRef()); })
		.EmplaceStep<FSessionRegisterPlayersStep>(SessionName, &Players, bWasInvited)
		.EmplaceStep<FSessionUnregisterPlayersStep>(SessionName, &Players)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);

	RunToCompletion();
}
