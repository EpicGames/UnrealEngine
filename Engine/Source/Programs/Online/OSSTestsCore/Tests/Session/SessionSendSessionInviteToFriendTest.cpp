// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"
#include "TestUtilities.h"

#include "Online/OnlineSessionNames.h"

#include "OnlineSubsystemCatchHelper.h"

#include "Helpers/Identity/IdentityGetUniquePlayerIdHelper.h"
#include "Helpers/Session/SessionUnregisterPlayerHelper.h"
#include "Helpers/Session/SessionCreateSessionHelper.h"
#include "Helpers/Session/SessionDestroySessionHelper.h"
#include "Helpers/Session/SessionSendSessionInviteToFriendHelper.h"

#define SESSION_TAG "[suite_session]"

#define EG_SESSION_SENDSESSIONINVITETOFRIEND_TAG SESSION_TAG "[sendsessioninvitetofriend]"
#define SESSION_TEST_CASE(x, ...) ONLINESUBSYSTEM_TEST_CASE(x, SESSION_TAG __VA_ARGS__)

SESSION_TEST_CASE("Verify calling Session SendSessionInviteToFriend with valid inputs returns the expected result(Success Case)", EG_SESSION_SENDSESSIONINVITETOFRIEND_TAG)
{
	int32 LocalUserNum = 0;
	int32 TargetUserNum = 1;
	int32 PublicConnections = 2;
	int32 PrivateConnections = 1;
	int32 NumUsersToImplicitLogin = 2;
	FUniqueNetIdPtr LocalUserId = nullptr;
	FUniqueNetIdPtr TargetUserId = nullptr;
	FName SessionName = FName(FString::Printf(TEXT("TestSession_%s"), *FTestUtilities::GetUniqueTestString()));
	FOnlineSessionSetting GamemodeSetting(TEXT("FORTEMPTY"), EOnlineDataAdvertisementType::ViaOnlineService);
	FOnlineSessionSettings SessionSettings;
	SessionSettings.bShouldAdvertise = true;
	SessionSettings.bAllowJoinInProgress = true;
	SessionSettings.NumPublicConnections = PublicConnections;
	SessionSettings.NumPrivateConnections = PrivateConnections;
	SessionSettings.Settings.Add(SETTING_GAMEMODE, GamemodeSetting);
	GetLoginPipeline(NumUsersToImplicitLogin)
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(LocalUserNum, [&LocalUserId](FUniqueNetIdPtr InUserId) {LocalUserId = InUserId; })
		.EmplaceStep<FIdentityGetUniquePlayerIdStep>(TargetUserNum, [&TargetUserId](FUniqueNetIdPtr InUserId) {TargetUserId = InUserId; })
		.EmplaceStep<FSessionCreateSessionStep>(&LocalUserId, SessionName, SessionSettings)
		.EmplaceStep<FSessionUnregisterPlayerStep>(SessionName, &TargetUserId)
		.EmplaceStep<FSessionSendSessionInviteToFriendStep>(&LocalUserId, SessionName, &TargetUserId)
		.EmplaceStep<FSessionUnregisterPlayerStep>(SessionName, &TargetUserId)
		.EmplaceStep<FSessionSendSessionInviteToFriendStep>(LocalUserNum, SessionName, &TargetUserId)
		.EmplaceStep<FSessionDestroySessionStep>(SessionName);
	RunToCompletion();
}
