// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;
using System.Collections.Generic;
using System.Linq;

public class OnlineTestsCore : ModuleRules
{
	public OnlineTestsCore(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = false;
		CppStandard = CppStandardVersion.EngineDefault;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"Projects",
				"EngineSettings",
				"EOSSDK",
				"EOSShared",
				"OnlineSubsystem",
				"OnlineServicesInterface",
				"OnlineServicesCommon",				
				"OnlineServicesOSSAdapter",
				"SSL",
				"Json",
				"JsonUtilities",
				"HTTP"
			}
		);

		// Disable external auth if target doesn't define it.
		if (!Target.GlobalDefinitions.Contains("ONLINETESTS_USEEXTERNAUTH=1"))
		{
			PublicDefinitions.Add(String.Format("ONLINETESTS_USEEXTERNAUTH=0"));
		}
	}
}


