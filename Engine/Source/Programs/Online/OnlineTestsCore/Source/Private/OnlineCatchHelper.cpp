// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"

#include "Algo/AllOf.h"
#include "Algo/ForEach.h"
#include "Algo/Sort.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Helpers/Auth/AuthLogin.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "Online/OnlineServicesRegistry.h"

// Make sure there are registered input devices for N users and fire
// OnInputDeviceConnectionChange delegate for interested online service code.
void EnsureLocalUserCount(uint32 NumUsers)
{
	TArray<FPlatformUserId> Users;
	IPlatformInputDeviceMapper::Get().GetAllActiveUsers(Users);

	const uint32 ActiveUserCount = Users.Num();

	for (uint32 Index = ActiveUserCount; Index < NumUsers; ++Index)
	{
		int32 Result = Users.Find(FPlatformMisc::GetPlatformUserForUserIndex(Index));
		if (Result < 0)
		{
			IPlatformInputDeviceMapper::Get().Internal_MapInputDeviceToUser(
				FInputDeviceId::CreateFromInternalId(Index),
				FPlatformMisc::GetPlatformUserForUserIndex(Index),
				EInputDeviceConnectionState::Connected);
		}
	}
}

TArray<TFunction<void()>>* GetGlobalInitalizers()
{
	static TArray<TFunction<void()>> gInitalizersToCallInMain;
	return &gInitalizersToCallInMain;
}

void OnlineTestBase::ConstructInternal(FString ServiceName, UE::Online::EOnlineServices InServiceType)
{
	Service = ServiceName;
	ServiceType = InServiceType;
}

OnlineTestBase::OnlineTestBase()
	: ServiceType(EOnlineServices::None)
	, Pipeline(MakeShared<FTestPipeline>(Driver.MakePipeline()))
{
	// handle most cxn in ConstructInternal
}

OnlineTestBase::~OnlineTestBase()
{

}

FString OnlineTestBase::GetService() const
{
	return Service;
}

EOnlineServices OnlineTestBase::GetServiceType() const
{
	return ServiceType;
}

IOnlineServicesPtr OnlineTestBase::GetServices() const
{
	return UE::Online::FOnlineServicesRegistry::Get().GetNamedServicesInstance(ServiceType, NAME_None, NAME_None);
}

bool OnlineTestBase::DeleteAccounts(int32 TestAccountIndex) const
{
#if ONLINETESTS_USEEXTERNAUTH
	return CustomDeleteAccounts(TestAccountIndex);
#else // ONLINETESTS_USEEXTERNAUTH
	return false;
#endif // ONLINETESTS_USEEXTERNAUTH
}

void OnlineTestBase::DestroyCurrentServiceModule() const
{
	UE::Online::FOnlineServicesRegistry::Get().DestroyNamedServicesInstance(ServiceType, NAME_None, NAME_None);
}

bool OnlineTestBase::ResetAccountStatus(int32 TestAccountIndex) const
{
#if ONLINETESTS_USEEXTERNAUTH
	return CustomResetAccounts(TestAccountIndex);
#else // ONLINETESTS_USEEXTERNAUTH
	return false;
#endif // ONLINETESTS_USEEXTERNAUTH
}

TArray<FString> GetServiceModules()
{
	TArray<FString> Modules;

	for (const OnlineAutoReg::FApplicableServicesConfig& Config : OnlineAutoReg::GetApplicableServices())
	{
		for (const FString& Module : Config.ModulesToLoad)
		{
			Modules.AddUnique(Module);
		}
	}

	return Modules;
}

void OnlineTestBase::LoadServiceModules()
{
	for (const FString& Module : GetServiceModules())
	{
		FModuleManager::LoadModulePtr<IModuleInterface>(*Module);
	}
}

void OnlineTestBase::UnloadServiceModules()
{
	const TArray<FString>& Modules = GetServiceModules();
	// Shutdown in reverse order
	for (int Index = Modules.Num() - 1; Index >= 0; --Index)
	{
		if (IModuleInterface* Module = FModuleManager::Get().GetModule(*Modules[Index]))
		{
			Module->ShutdownModule();
		}
	}
}

TArray<UE::Online::FAuthLogin::Params> OnlineTestBase::GetIniCredentials(int32 TestAccountIndex) const
{
	FString LoginCredentialCategory = GetLoginCredentialCategory();
	TArray<FString> LoginCredentialsRaw;
	GConfig->GetArray(*LoginCredentialCategory, TEXT("Credentials"), LoginCredentialsRaw, GEngineIni);

	if (TestAccountIndex >= LoginCredentialsRaw.Num())
	{
		UE_LOG_ONLINETESTS(Error, TEXT("Attempted to GetCredentials for more than we have stored! Add more credentials to the DefaultEngine.ini for OnlineTests"));
		REQUIRE(TestAccountIndex >= LoginCredentialsRaw.Num());
		return TArray<UE::Online::FAuthLogin::Params>();
	}

	TArray<UE::Online::FAuthLogin::Params> AuthLoginParams;

	TArray<FString> LoginCredentialSplit;
	LoginCredentialsRaw[TestAccountIndex].ParseIntoArray(LoginCredentialSplit, TEXT(","), true);

	INFO(*FString::Printf(TEXT("Logging in with type %s, id %s, password %s"), *LoginCredentialSplit[0], LoginCredentialSplit.Num() > 1 ? *LoginCredentialSplit[1] : TEXT("UNSET"), LoginCredentialSplit.Num() > 2 ? *LoginCredentialSplit[2] : TEXT("UNSET")));
	FAuthLogin::Params Params;
	Params.CredentialsType = *LoginCredentialSplit[0];
	if (LoginCredentialSplit.Num() > 1)
	{
		Params.CredentialsId = LoginCredentialSplit[1];
	}
	if (LoginCredentialSplit.Num() > 2)
	{
		Params.CredentialsToken.Set<FString>(LoginCredentialSplit[2]);
	}
	Params.PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(AuthLoginParams.Num());

	AuthLoginParams.Add(Params);

	return AuthLoginParams;
}

TArray<UE::Online::FAuthLogin::Params> OnlineTestBase::GetCredentials(int32 TestAccountIndex, int32 NumUsers) const
{
#if ONLINETESTS_USEEXTERNAUTH
	return CustomCredentials(TestAccountIndex, NumUsers);
#else // ONLINETESTS_USEEXTERNAUTH
	return GetIniCredentials(TestAccountIndex);
#endif // ONLINETESTS_USEEXTERNAUTH
}

FString OnlineTestBase::GetLoginCredentialCategory() const
{
	return FString::Printf(TEXT("LoginCredentials %s"), *Service);
}

void OnlineTestBase::AssignLoginUsers(int32 LocalUserId, FAccountId& OutAccountId) const
{
	IOnlineServicesPtr OnlineServices = GetServices();
	UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByPlatformUserId> UserId = OnlineServices->GetAuthInterface()->GetLocalOnlineUserByPlatformUserId({ FPlatformMisc::GetPlatformUserForUserIndex(LocalUserId) });
	REQUIRE(UserId.IsOk());
	CHECK(UserId.TryGetOkValue() != nullptr);
	OutAccountId = UserId.TryGetOkValue()->AccountInfo->AccountId;
}

FTestPipeline& OnlineTestBase::GetLoginPipeline(std::initializer_list<std::reference_wrapper<FAccountId>> AccountIds) const
{
	return GetLoginPipeline(0, AccountIds);
}

FTestPipeline& OnlineTestBase::GetLoginPipeline(uint32 TestAccountIndex, std::initializer_list<std::reference_wrapper<FAccountId>> AccountIds) const
{
	// Don't call GetLoginPipeline more than once per test
	REQUIRE(NumLocalUsers == -1);
	NumLocalUsers = AccountIds.size();

	bool bUseAutoLogin = false;
	bool bUseImplicitLogin = false;
	FString LoginCredentialCategory = GetLoginCredentialCategory();
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseAutoLogin"), bUseAutoLogin, GEngineIni);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseImplicitLogin"), bUseImplicitLogin, GEngineIni);
	
	// Make sure input delegates are fired for adding the required user count.
	EnsureLocalUserCount(NumLocalUsers);

	if (bUseImplicitLogin)
	{
		// Users are expected to already be valid.
		int32 Index = 0;
		for (FAccountId& AccountId : AccountIds)
		{
			AssignLoginUsers(Index, AccountId);
			++Index;
		}
	}
	else if (bUseAutoLogin)
	{
		// todo
		// NumLocalUsers = 1;
		// Pipeline.EmplaceStep<FAuthAutoLoginStep>(0);
	}
	else if (NumLocalUsers > 0)
	{
		TestAccountID = TestAccountIndex;

		TArray<UE::Online::FAuthLogin::Params> AuthLoginParams = GetCredentials(TestAccountIndex, NumLocalUsers);
		for (UE::Online::FAuthLogin::Params AuthLoginParam : AuthLoginParams)
		{
			Pipeline->EmplaceStep<FAuthLoginStep>(MoveTemp(AuthLoginParam));
		}

		// Perform login so we can bulk assign users in the next step.
		RunToCompletion(false);

		int32 Index = 0;
		for (FAccountId& AccountId : AccountIds)
		{
			UE_LOG_ONLINETESTS(Verbose, TEXT("Account: %s, used InrernalId: %d"), *AuthLoginParams[Index].CredentialsId, AuthLoginParams[Index].PlatformUserId.GetInternalId());
			AssignLoginUsers(AuthLoginParams[Index].PlatformUserId.GetInternalId(), AccountId);
			++Index;
		}
	}

	// Return a fresh pipeline so the logins added by GetLoginPipeline don't execute again.
	Pipeline = MakeShared<FTestPipeline>(Driver.MakePipeline());
	return *Pipeline;
}

FTestPipeline& OnlineTestBase::GetPipeline() const
{
	return GetLoginPipeline(0, {});
}

void OnlineTestBase::RunToCompletion(bool bLogout) const
{
	bool bUseAutoLogin = false;
	bool bUseImplicitLogin = false;
	FString LoginCredentialCategory = GetLoginCredentialCategory();
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseAutoLogin"), bUseAutoLogin, GEngineIni);
	GConfig->GetBool(*LoginCredentialCategory, TEXT("UseImplicitLogin"), bUseImplicitLogin, GEngineIni);
	if (bLogout)
	{
		if (bUseImplicitLogin)
		{
			// Users are expected to already be valid.
		}
		else if (bUseAutoLogin)
		{
			// todo
			// NumLocalUsers = 1;
			// Pipeline.EmplaceStep<FAuthAutoLoginStep>(0);
		}
		else if(NumLocalUsers > 0)
		{
			TArray<UE::Online::FAuthLogin::Params> AuthLoginParams = GetCredentials(TestAccountID, NumLocalUsers);
			for (UE::Online::FAuthLogin::Params AuthLoginParam : AuthLoginParams)
			{
				Pipeline->EmplaceStep<FAuthLogoutStep>(AuthLoginParam.PlatformUserId);
			}
		}
	}

	FPipelineTestContext TestContext = FPipelineTestContext(FName(GetService()), GetServiceType());
	REQUIRE(Driver.AddPipeline(MoveTemp(*Pipeline), TestContext)); // If this fails, we were unable to find the service that is being passed by GetService
	Pipeline = nullptr;
	Driver.RunToCompletion();

	if (bLogout)
	{
		// This is to fix "XXX online services is still been referenced after shutting down". Also need to 
		// make sure to not cache online services shared ptr or interface ptr in your test case scope
		DestroyCurrentServiceModule();
	}
}

TArray<OnlineAutoReg::FApplicableServicesConfig> OnlineAutoReg::GetApplicableServices()
{
	static TArray<FApplicableServicesConfig> ServicesConfig = 
	[]()
	{
		TArray<FApplicableServicesConfig> ServicesConfigInit;
		if (const TCHAR* CmdLine = FCommandLine::Get())
		{
			FString Values;
			TArray<FString> ServicesTags;
			if (FParse::Value(CmdLine, TEXT("-Services="), Values, false))
			{
				Values.ParseIntoArray(ServicesTags, TEXT(","));
			}

			if (ServicesTags.IsEmpty())
			{
				GConfig->GetArray(TEXT("OnlineServicesTests"), TEXT("DefaultServices"), ServicesTags, GEngineIni);
			}

			for (const FString& ServicesTag : ServicesTags)
			{
				FString ConfigCategory = FString::Printf(TEXT("OnlineServicesTests %s"), *ServicesTag);
				FApplicableServicesConfig Config;
				Config.Tag = ServicesTag;

				FString ServicesType;
				GConfig->GetString(*ConfigCategory, TEXT("ServicesType"), ServicesType, GEngineIni);
				GConfig->GetArray(*ConfigCategory, TEXT("ModulesToLoad"), Config.ModulesToLoad, GEngineIni);

				LexFromString(Config.ServicesType, *ServicesType);
				if (Config.ServicesType != EOnlineServices::None)
				{
					ServicesConfigInit.Add(MoveTemp(Config));
				}
			}
		}

		return ServicesConfigInit;
	}();

	return ServicesConfig;
}

bool OnlineAutoReg::CheckAllTagsIsIn(const TArray<FString>& TestTags, const TArray<FString>& InputTags)
{
	if (InputTags.Num() == 0)
	{
		return false;
	}

	if (InputTags.Num() > TestTags.Num())
	{
		return false;
	}

	bool bAllInputTagsInTestTags = Algo::AllOf(InputTags, [&TestTags](const FString& CheckTag) -> bool
	{
		auto CheckStringCaseInsenstive = [&CheckTag](const FString& TestString) -> bool
		{
			return TestString.Equals(CheckTag, ESearchCase::IgnoreCase);
		};

		if (TestTags.ContainsByPredicate(CheckStringCaseInsenstive))
		{
			return true;
		}

		return false;
	});

	return bAllInputTagsInTestTags;
}

bool OnlineAutoReg::CheckAllTagsIsIn(const TArray<FString>& TestTags, const FString& RawTagString)
{
	TArray<FString> InputTags;
	RawTagString.ParseIntoArray(InputTags, TEXT(","));
	Algo::ForEach(InputTags, [](FString& String)
	{
		String.TrimStartAndEndInline();
		String.RemoveFromStart("[");
		String.RemoveFromEnd("]");
	});
	return CheckAllTagsIsIn(TestTags, InputTags);
}

FString OnlineAutoReg::GenerateTags(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, const TCHAR* InTag)
{
	//Copy String here for ease-of-manipulation
	FString RawInTag = InTag;

	TArray<FString> TestTagsArray;
	RawInTag.ParseIntoArray(TestTagsArray, TEXT("]"));
	Algo::ForEach(TestTagsArray, [](FString& String)
	{
		String.TrimStartAndEndInline();
		String.RemoveFromStart("[");
	});
	Algo::Sort(TestTagsArray);

	// Search if we need to append [!mayfail] tag to indicate to 
	// catch2 this test is in a in-development phase and failures 
	// should be ignored.
	for (const FString& FailableTags : SkippableTags.MayFailTags)
	{
		if (CheckAllTagsIsIn(TestTagsArray, FailableTags))
		{
			RawInTag.Append(TEXT("[!mayfail]"));
			break;
		}
	}

	// Search if we need to append [!shouldfail] tag to indicate to 
	// catch2 this test should fail, and if it ever passes we should
	// should fail.
	for (const FString& FailableTags : SkippableTags.ShouldFailTags)
	{
		if (CheckAllTagsIsIn(TestTagsArray, FailableTags))
		{
			RawInTag.Append(TEXT("[!shouldfail]"));
			break;
		}
	}

	return FString::Printf(TEXT("[%s] %s"), *ServiceName, *RawInTag);
}

bool OnlineAutoReg::ShouldDisableTest(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, const TCHAR* InTag)
{
	//Copy String here for ease-of-manipulation
	const FString RawInTag = InTag;

	TArray<FString> TestTagsArray;
	RawInTag.ParseIntoArray(TestTagsArray, TEXT("]"));
	Algo::ForEach(TestTagsArray, [](FString& String)
	{
		String.TrimStartAndEndInline();
		String.RemoveFromStart("[");
	});
	Algo::Sort(TestTagsArray);

	// If we contain [!<service>] it means we shouldn't run this
	// test against this service.
	if (RawInTag.Contains("!" + ServiceName))
	{
		return true;
	}

	// Check for exclusive runs
	for (const OnlineAutoReg::FApplicableServicesConfig& Config : GetApplicableServices())
	{
		const FString& ServiceTag = Config.Tag;
		if (ServiceName.Equals(ServiceTag, ESearchCase::IgnoreCase))
		{
			continue;
		}

		// If we contain [.NULL] and we're running with [EOS] we shouldn't
		// generate a test for [EOS] here.
		if (RawInTag.Contains("." + ServiceTag))
		{
			return true;
		}
	}

	// If we contain tags from config it means 
	// we shouldn't run this test
	for (const FString& DisableTag : SkippableTags.DisableTestTags)
	{
		if (CheckAllTagsIsIn(TestTagsArray, DisableTag))
		{
			return true;
		}
	}

	// We should run the test!
	return false;
}

//We check all applicable services to see if `TagsToCheck` is present in any TestReporting confugraitons.
//Returns true if TagsToCheck is present in any [TestReporting <Platform>] section.
bool OnlineAutoReg::ShouldSkipTest(const FString& TagsToCheck) {
	for (const OnlineAutoReg::FApplicableServicesConfig& Config : GetApplicableServices())
	{
		const FString& ServiceTag = Config.Tag;
		FString ReportingCategory = FString::Printf(TEXT("TestReporting %s"), *ServiceTag);
		FReportingSkippableTags SkippableTags;
		GConfig->GetArray(*ReportingCategory, TEXT("MayFailTestTags"), SkippableTags.MayFailTags, GEngineIni);
		GConfig->GetArray(*ReportingCategory, TEXT("ShouldFailTestTags"), SkippableTags.ShouldFailTags, GEngineIni);
		GConfig->GetArray(*ReportingCategory, TEXT("DisableTestTags"), SkippableTags.DisableTestTags, GEngineIni);

		auto NewTags = StringCast<ANSICHAR>(*TagsToCheck);

		// If we have tags present indicating we should exit the test
		if (ShouldDisableTest(ServiceTag, SkippableTags, ANSI_TO_TCHAR(NewTags.Get())))
		{
			return true;
		}
	}
	return false;
}

void OnlineAutoReg::CheckRunningTestSkipOnTags() {
	FString CurrentRunningTestTags(Catch::getActiveTestTags().c_str());
	if (OnlineAutoReg::ShouldSkipTest(CurrentRunningTestTags)) {
		SKIP("Test skipped due to TestReporting DisableTestTags");
	}
}

// This code is kept identical to Catch internals so that there is as little deviation from OSS_TESTS and Online_OSS_TESTS as possible
OnlineAutoReg::OnlineAutoReg(OnlineTestConstructor TestCtor, Catch::SourceLineInfo LineInfo, const char* Name, const char* Tags, const char* AddlOnlineInfo)
{
	auto GlobalInitalizersPtr = GetGlobalInitalizers();
	ensure(GlobalInitalizersPtr);
	GlobalInitalizersPtr->Add([=]() -> void
	{
		for (const OnlineAutoReg::FApplicableServicesConfig& Config : GetApplicableServices())
		{
			const FString& ServiceTag = Config.Tag;
			FString ReportingCategory = FString::Printf(TEXT("TestReporting %s"), *ServiceTag);
			FReportingSkippableTags SkippableTags;
			GConfig->GetArray(*ReportingCategory, TEXT("MayFailTestTags"), SkippableTags.MayFailTags, GEngineIni);
			GConfig->GetArray(*ReportingCategory, TEXT("ShouldFailTestTags"), SkippableTags.ShouldFailTags, GEngineIni);
			GConfig->GetArray(*ReportingCategory, TEXT("DisableTestTags"), SkippableTags.DisableTestTags, GEngineIni);

			auto NewName = StringCast<ANSICHAR>(*FString::Printf(TEXT("[%s] %s"), *ServiceTag, ANSI_TO_TCHAR(Name)));
			auto NewTags = StringCast<ANSICHAR>(*GenerateTags(ServiceTag, SkippableTags, ANSI_TO_TCHAR(Tags)));

			// If we have tags present indicating we should not enable the test at all
			if (ShouldDisableTest(ServiceTag, SkippableTags, ANSI_TO_TCHAR(NewTags.Get())))
			{
				continue;
			}

			// TestCtor will create a new instance of the test we are calling- ConstructInternal is separate so that we can pass any arguments we want instead of baking them into the macro
			OnlineTestBase* NewTest = TestCtor();
			NewTest->ConstructInternal(ServiceTag, Config.ServicesType);

			// This code is lifted from Catch internals to register a test
			Catch::getMutableRegistryHub().registerTest(Catch::makeTestCaseInfo(
				std::string(Catch::StringRef()),  // Used for testing a static method instead of a function- not needed since we're passing an ITestInvoker macro
				Catch::NameAndTags{ NewName.Get(), NewTags.Get() },
				LineInfo),
				Catch::Detail::unique_ptr(NewTest) // This is taking the ITestInvoker macro and will call invoke() to run the test; 
			);
		}
	});
}
