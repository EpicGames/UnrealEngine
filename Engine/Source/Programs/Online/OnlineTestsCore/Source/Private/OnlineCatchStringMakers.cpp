// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchStringMakers.h"

#include "Containers/StringConv.h"
#include "Online/OnlineError.h"

namespace Catch
{
	std::string StringMaker<UE::Online::FOnlineError>::convert(UE::Online::FOnlineError const& Error)
	{
		auto ErrorRepresentation = StringCast<UTF8CHAR>(*Error.GetLogString());
		return std::string((const std::string::value_type*)ErrorRepresentation.Get());
	}
}

