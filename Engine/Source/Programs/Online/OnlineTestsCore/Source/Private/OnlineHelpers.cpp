// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "Helpers/OnlineHelpers.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"

namespace UE::TestCommon {

void Tick()
{
	double TickTime = FTimespan::FromMilliseconds(TickFrequencyMs).GetTotalSeconds();
	FTSTicker::GetCoreTicker().Tick(TickTime);
	FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
}

/* UE::TestCommon */ }