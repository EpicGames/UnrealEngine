// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestDriver.h"

#include "TestHarness.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"
#include "Online/OnlineExecHandler.h"
#include "Online/OnlineServicesRegistry.h"

FString FTestPipeline::InfoString() const
{
	const double CurrentTime = FPlatformTime::Seconds();

	TStringBuilder<128> Info;
	Info.Append(TEXT("TestSteps:"));
	Info.Append(FString::FromInt(TestSteps.Num()));
	Info.Append(TEXT(" "));
	Info.Append(TEXT("CompletedSteps:"));
	Info.Append(FString::FromInt(CompletedSteps.Num()));
	Info.Append(TEXT(" "));
	Info.Append(TEXT("TimedOutSteps:"));
	Info.Append(FString::FromInt(TimedoutSteps.Num()));
	Info.Append(TEXT(" ")),
	Info.Append(TEXT("DeletePostReleaseSteps:"));
	Info.Append(FString::FromInt(DeletePostReleaseSteps.Num()));
	Info.Append(TEXT(" "));
	Info.Append(TEXT("PipelineTime:"));
	Info.Append(FString::FromInt(FTimespan(CurrentTime - PipelineStartTime).GetTotalSeconds()));
	Info.Append(TEXT("s "));
	Info.Append(TEXT("ServiceTickSum:"));
	Info.Append(FString::FromInt(FTimespan(ServiceTickSum).GetTotalMilliseconds()));
	Info.Append(TEXT("ms "));
	Info.Append(TEXT("ServiceTickCount:"));
	Info.Append(FString::FromInt(ServiceTickCount));

	const FString InfoString = Info.ToString();

	return InfoString;
}

FString FTestDriver::TimeoutFailedTestInfo() const
{
	TStringBuilder<128> FailedInfo;
	FailedInfo.Append(TEXT("[Error] Test driver encountered a timeout during test execution."));
	FailedInfo.Append(TEXT("   TestFailedOnStepNum:"));
	FailedInfo.Append(FString::FromInt(this->FailedStepNum));

	const FString FailedInfoString = FailedInfo.ToString();

	return FailedInfoString;
}

void FTestPipeline::operator()(const IOnlineServicesPtr& Services)
{
	const double CurrentTime = FPlatformTime::Seconds();

	if (TestSteps.IsEmpty())
	{
		Driver.MarkComplete(Services);
		return;
	}

	FStepPtr& CurrentStep = TestSteps.HeapTop();
	switch (CurrentStep->Tick(Services))
	{

	case FStep::EContinuance::Done:
		// Retain the completed steps for those which have long term notify handlers.
		CompletedSteps.Emplace(MoveTemp(CurrentStep));
		TestSteps.RemoveAt(0);
		break;

	default:
		if (FTimespan::FromSeconds(CurrentTime - PipelineStartTime).GetTicks() >= PipelineTimeout.GetTicks())
		{
			Driver.SetDriverTimedOut(true);
			if (CurrentStep->IsOptional())
			{
				// Mark only this step as timed out.
				// Retain the timed-out steps since the callback has yet to be handled.
				TimedoutSteps.Emplace(MoveTemp(CurrentStep));
				TestSteps.RemoveAt(0);
			}
			else
			{
				// Mark all remaining steps as timed out.
				while (!TestSteps.IsEmpty())
				{
					FTestPipeline::FStepPtr& ExistingStep = TestSteps.HeapTop();
					// Retain the timed-out steps since the callback may not have been handled.
					TimedoutSteps.Emplace(MoveTemp(ExistingStep));
					TestSteps.RemoveAt(0);
				}
				Driver.MarkComplete(Services);
			}
		}
		break;
	}
}

void FTestPipeline::OnPreRelease()
{
	if (!TestSteps.IsEmpty() || !TimedoutSteps.IsEmpty())
	{
		this->Driver.FailedStepNum = CompletedSteps.Num()+1;
	}

	auto ReleaseList = [this](TArray<FStepPtr>& Steps)
	{
		while (!Steps.IsEmpty())
		{
			FStepPtr& ExistingStep = Steps.Top();
			if (ExistingStep->RequiresDeletePostRelease())
			{
				ExistingStep->OnPreRelease();
				DeletePostReleaseSteps.Emplace(MoveTemp(ExistingStep));
			}
			Steps.RemoveAt(0);
		}
	};

	ReleaseList(TestSteps);
	ReleaseList(CompletedSteps);
	ReleaseList(TimedoutSteps);
}

void FTestPipeline::EvaluatePlatformTickTime(double&& TickTime)
{
	ServiceTickSum += FTimespan::FromSeconds(TickTime).GetTicks();
	++ServiceTickCount;

	if (EvaluateTickConfig.bEvaluateTickCheckActive)
	{
		if (ServiceTickCount >= EvaluateTickConfig.MinimumTickCount)
		{
			FTimespan AverageTick = ServiceTickSum.GetTicks() / ServiceTickCount;
			FTimespan ExpectedAverageTick = EvaluateTickConfig.ExpectedAverageTick.GetTicks();
			CHECK(AverageTick <= ExpectedAverageTick);
		}

		FTimespan Tick = FTimespan::FromSeconds(TickTime);
		FTimespan AbsoluteMaximumTick = EvaluateTickConfig.AbsoluteMaximumTick.GetTicks();
		CHECK(Tick <= AbsoluteMaximumTick); 
	}
}

FTestDriver::~FTestDriver()
{
	// Mark all remaining services as complete.
	ForeachServicesInstance([this](const IOnlineServicesPtr&  OnlineService, FTestPipeline&)
	{
		MarkComplete(OnlineService);
	});

	FlushCompleted();
}

bool FTestDriver::AddPipeline(FTestPipeline&& Pipeline, const FPipelineTestContext& TestContext)
{
	IOnlineServicesPtr OnlineServices = UE::Online::FOnlineServicesRegistry::Get().GetNamedServicesInstance(TestContext.InitOptions.ServiceType, NAME_None, NAME_None);
	if (OnlineServices == nullptr)
	{
		return false;
	}
	
	ServicesInstances.Emplace(OnlineServices, MoveTemp(Pipeline));
	return true;
}

void FTestDriver::MarkComplete(IOnlineServicesPtr Key)
{
	CompletedInstances.Emplace(Key);
}

void FTestDriver::RunToCompletion()
{
	LastTickTime = FPlatformTime::Seconds();

	ForeachServicesInstance([](const IOnlineServicesPtr&, FTestPipeline& Func)
	{
		Func.Start();
	});

	while (!ServicesInstances.IsEmpty())
	{
		FPlatformProcess::Sleep(TICK_DURATION.GetTotalSeconds());
		ForeachServicesInstance([this](const IOnlineServicesPtr& Services, FTestPipeline& CurrentPipeline)
		{
			const double BeforeTick = FPlatformTime::Seconds();
			{ // Scope for INFO

				double TickTime = BeforeTick - LastTickTime;
				FTSTicker::GetCoreTicker().Tick(TickTime);
				// need to process game thread tasks at least once a tick
				FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
				LastTickTime = BeforeTick;
			}

			const double AfterTick = FPlatformTime::Seconds();
			CurrentPipeline.EvaluatePlatformTickTime(AfterTick - BeforeTick);
		});
		ForeachServicesInstance([](const IOnlineServicesPtr& Services, FTestPipeline& Func)
		{
			UE_LOGF(LogOnlineServices, Log, "INFO %ls", *Func.InfoString());
			Func(Services);
		});
		FlushCompleted();
	}

	if (bDidTimeout)
	{
		FAIL_CHECK(TCHAR_TO_ANSI (*TimeoutFailedTestInfo()));
	}
}

void FTestDriver::FlushCompleted()
{
	for (const IOnlineServicesPtr& Key : CompletedInstances)
	{
		ServicesInstances.Find(Key)->OnPreRelease();
		ServicesInstances.Remove(Key);
	}
	CompletedInstances.Empty();
}