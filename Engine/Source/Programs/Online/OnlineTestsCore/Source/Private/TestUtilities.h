// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TestHarness.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"

class FTestUtilities
{
public:
	static FString GetUniqueTestString()
	{
		FString UniqueString = FGuid().ToString();

		return UniqueString;
	}

	static int32 GetUniqueTestInteger()
	{
		int32 Rand = FMath::RandHelper(INT_MAX);
		
		return Rand;
	}
};
