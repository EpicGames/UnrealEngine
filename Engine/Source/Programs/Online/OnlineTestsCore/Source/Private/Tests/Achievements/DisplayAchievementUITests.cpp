// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Achievements/QueryAchievementStatesHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_DISPLAYACHIEVEMENTUI_TAG ACHIEVEMENTS_TAG "[displayachievementui][.NULL]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Display Achievement UI (Invalid User)", EG_ACHIEVEMENTS_DISPLAYACHIEVEMENTUI_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem) 
			{
				UE::Online::FDisplayAchievementUI::Params OpParams;

				UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
				UE::Online::TOnlineResult<UE::Online::FDisplayAchievementUI> Result = AchievementsInterface->DisplayAchievementUI(MoveTemp(OpParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidUser());
			});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Display Achievement UI (Invalid State)",  EG_ACHIEVEMENTS_DISPLAYACHIEVEMENTUI_TAG)
{
	FAccountId AccountId;
	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem) 
		{
			UE::Online::FDisplayAchievementUI::Params Params;
			Params.LocalAccountId = AccountId;

			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FDisplayAchievementUI> Result = AchievementsInterface->DisplayAchievementUI(MoveTemp(Params));
			REQUIRE(Result.IsError());
			CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidState());
		});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Display Achievement UI (Not Found)", EG_ACHIEVEMENTS_DISPLAYACHIEVEMENTUI_TAG)
{
	FAccountId AccountId;

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	
	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem) 
		{
			UE::Online::FDisplayAchievementUI::Params Params;
			Params.LocalAccountId = AccountId;
			Params.AchievementId = TEXT("UnknownAchievement");

			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FDisplayAchievementUI> Result = AchievementsInterface->DisplayAchievementUI(MoveTemp(Params));
			REQUIRE(Result.IsError());
			CHECK(Result.GetErrorValue() == UE::Online::Errors::NotFound());
		});
			
	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Display Achievement UI (Success)", EG_ACHIEVEMENTS_DISPLAYACHIEVEMENTUI_TAG)
{
	FAccountId AccountId;

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem) 
		{
			UE::Online::FDisplayAchievementUI::Params Params;
			Params.LocalAccountId = AccountId;
			Params.AchievementId = TEXT("test_unlockachievements");

			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FDisplayAchievementUI> Result = AchievementsInterface->DisplayAchievementUI(MoveTemp(Params));
			CHECK_OP(Result);
		});

	RunToCompletion();
}