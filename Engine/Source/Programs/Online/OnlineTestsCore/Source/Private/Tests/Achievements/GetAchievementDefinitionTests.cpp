// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_GETACHIEVEMENTDEFINITION_TAG ACHIEVEMENTS_TAG "[getachievementdefinition]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Get Achievement Definition (Invalid User)", EG_ACHIEVEMENTS_GETACHIEVEMENTDEFINITION_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::FGetAchievementDefinition::Params OpParams;

				UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
				UE::Online::TOnlineResult<UE::Online::FGetAchievementDefinition> Result = AchievementsInterface->GetAchievementDefinition(MoveTemp(OpParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidUser());
			});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement Definition (Invalid State)", EG_ACHIEVEMENTS_GETACHIEVEMENTDEFINITION_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	UE::Online::FGetAchievementDefinition::Params GetAchievementDefinitionParams;
	GetAchievementDefinitionParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceLambda([&GetAchievementDefinitionParams](const IOnlineServicesPtr& OnlineSubsystem)
		{
			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FGetAchievementDefinition> Result = AchievementsInterface->GetAchievementDefinition(MoveTemp(GetAchievementDefinitionParams));
			REQUIRE(Result.IsError());
			CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidState());
		});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement Definition (Not Found)", EG_ACHIEVEMENTS_GETACHIEVEMENTDEFINITION_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	UE::Online::FGetAchievementDefinition::Params GetParams;
	GetParams.LocalAccountId = AccountId;
	GetParams.AchievementId = TEXT("UnknownAchievement");

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceLambda([&GetParams](const IOnlineServicesPtr& OnlineSubsystem)
		{
			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FGetAchievementDefinition> Result = AchievementsInterface->GetAchievementDefinition(MoveTemp(GetParams));
			REQUIRE(Result.IsError());
			CHECK(Result.GetErrorValue() == UE::Online::Errors::NotFound());
		});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement Definitions (Success)", EG_ACHIEVEMENTS_GETACHIEVEMENTDEFINITION_TAG)
{
	FAccountId AccountId;
	TArray<FString> AchievementIds;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	UE::Online::FGetAchievementIds::Params GetParams;
	GetParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceLambda([&GetParams, &AccountId, &AchievementIds](const IOnlineServicesPtr& OnlineSubsystem)
		{
			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			UE::Online::TOnlineResult<UE::Online::FGetAchievementIds> Result = AchievementsInterface->GetAchievementIds(MoveTemp(GetParams));
			REQUIRE_OP(Result);
			AchievementIds = MoveTemp(Result.GetOkValue().AchievementIds);
		})
		.EmplaceLambda([&AccountId, &AchievementIds](const IOnlineServicesPtr& OnlineSubsystem)
		{
			REQUIRE(!AchievementIds.IsEmpty());
			for (const FString& AchievementId : AchievementIds)
			{
				UE::Online::FGetAchievementDefinition::Params GetParams;
				GetParams.LocalAccountId = AccountId;
				GetParams.AchievementId = AchievementId;

				UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
				const UE::Online::TOnlineResult<UE::Online::FGetAchievementDefinition> Result = AchievementsInterface->GetAchievementDefinition(MoveTemp(GetParams));
				REQUIRE_OP(Result);
			}
		});

	RunToCompletion();
}