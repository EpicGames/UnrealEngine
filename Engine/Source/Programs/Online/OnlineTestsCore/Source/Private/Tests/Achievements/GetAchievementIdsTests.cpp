// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_GETACHIEVEMENTIDS_TAG ACHIEVEMENTS_TAG "[getachievementids]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Get Achievement Ids (Invalid User)", EG_ACHIEVEMENTS_GETACHIEVEMENTIDS_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::FGetAchievementIds::Params OpParams;

				UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
				UE::Online::TOnlineResult<UE::Online::FGetAchievementIds> Result = AchievementsInterface->GetAchievementIds(MoveTemp(OpParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidUser());
			});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement Ids (Invalid State)", EG_ACHIEVEMENTS_GETACHIEVEMENTIDS_TAG)
{
	FAccountId AccountId;

	auto& LoginPipeline = GetLoginPipeline({ AccountId });

	UE::Online::FGetAchievementIds::Params OpParams;
	OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceLambda([&OpParams](const IOnlineServicesPtr& OnlineSubsystem)
		{
			IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			TOnlineResult<UE::Online::FGetAchievementIds> Result = AchievementsInterface->GetAchievementIds(MoveTemp(OpParams));
			REQUIRE(Result.IsError());
			CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidState());
		});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement Ids (Success)", EG_ACHIEVEMENTS_GETACHIEVEMENTIDS_TAG)
{
	FAccountId AccountId;

	auto& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	UE::Online::FGetAchievementIds::Params GetParams;
	GetParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceLambda([&GetParams](const IOnlineServicesPtr& OnlineSubsystem)
		{
			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FGetAchievementIds> Result = AchievementsInterface->GetAchievementIds(MoveTemp(GetParams));
			REQUIRE_OP(Result);
			CHECK(!Result.GetOkValue().AchievementIds.IsEmpty());
		});

	RunToCompletion();
}