// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Achievements/QueryAchievementStatesHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_GETACHIEVEMENTSTATE_TAG ACHIEVEMENTS_TAG "[getachievementstate]"
#define EG_ACHIEVEMENTS_GETACHIEVEMENTSTATE_SKIP_NOT_FOUND_TAG ACHIEVEMENTS_TAG "[getachievementstate][skip_achievement_not_found]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Get Achievement State (Invalid State)", EG_ACHIEVEMENTS_GETACHIEVEMENTSTATE_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	UE::Online::FGetAchievementState::Params GetParams;
	GetParams.LocalAccountId = AccountId;
	GetParams.AchievementId = TEXT("test_unlockachievements");

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceLambda([&GetParams](const IOnlineServicesPtr& OnlineSubsystem)
		{
			UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			const UE::Online::TOnlineResult<UE::Online::FGetAchievementState> Result = AchievementsInterface->GetAchievementState(MoveTemp(GetParams));
			REQUIRE(Result.IsError());
			CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidState());
		});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement State (Not Found)", EG_ACHIEVEMENTS_GETACHIEVEMENTSTATE_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	UE::Online::FGetAchievementState::Params GetParams;
	GetParams.LocalAccountId = AccountId;
	GetParams.AchievementId = TEXT("UnknownAchievement");

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceLambda([&GetParams](const IOnlineServicesPtr& OnlineSubsystem)
	{
		UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
		const UE::Online::TOnlineResult<UE::Online::FGetAchievementState> Result = AchievementsInterface->GetAchievementState(MoveTemp(GetParams));
		REQUIRE(Result.IsError());
		CHECK(Result.GetErrorValue() == UE::Online::Errors::NotFound());
	});

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement State (Success)", EG_ACHIEVEMENTS_GETACHIEVEMENTSTATE_SKIP_NOT_FOUND_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	UE::Online::FGetAchievementState::Params GetParams;
	GetParams.LocalAccountId = AccountId;
	GetParams.AchievementId = TEXT("test_unlockachievements");

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
	.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
	.EmplaceLambda([&GetParams](const IOnlineServicesPtr& OnlineSubsystem)
	{
		UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
		const UE::Online::TOnlineResult<UE::Online::FGetAchievementState> Result = AchievementsInterface->GetAchievementState(MoveTemp(GetParams));
		CHECK_OP(Result);
	});

	RunToCompletion();
}