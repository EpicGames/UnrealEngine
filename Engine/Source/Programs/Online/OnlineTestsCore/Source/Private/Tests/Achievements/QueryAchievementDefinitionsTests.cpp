// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_QUERYACHIEVEMENTDEFINITIONS_TAG ACHIEVEMENTS_TAG "[queryachievementdefinitions]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Query Definitions (Invalid User)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTDEFINITIONS_TAG)
{
	FAccountId AccountId;

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.ExpectedError = FQueryAchievementDefinitionsHelper::ResultType(UE::Online::Errors::InvalidUser());

	GetLoginPipeline({ AccountId })
		.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Query Definitions (Success)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTDEFINITIONS_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams));
	
	RunToCompletion();
}