// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Achievements/QueryAchievementStatesHelper.h"
#include "Helpers/Achievements/UnlockAchievementsHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_QUERYACHIEVEMENTSTATES_TAG ACHIEVEMENTS_TAG "[queryachievementstates]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Query Achievement States (Invalid User)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTSTATES_TAG)
{
	FAccountId AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.ExpectedError = FQueryAchievementStatesHelper::ResultType(UE::Online::Errors::InvalidUser());

	GetLoginPipeline({ AccountId })
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Query Achievement States (Invalid State)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTSTATES_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.ExpectedError = FQueryAchievementStatesHelper::ResultType(UE::Online::Errors::InvalidState());
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Query Achievement States (Success)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTSTATES_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	
	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Get Achievement State (Invalid User)", EG_ACHIEVEMENTS_QUERYACHIEVEMENTSTATES_TAG)
{
	FAccountId AccountId;

	GetLoginPipeline({ AccountId })
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::FGetAchievementState::Params Params;
				Params.AchievementId = TEXT("test_unlockachievements");

				UE::Online::IAchievementsPtr AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
				UE::Online::TOnlineResult<UE::Online::FGetAchievementState> Result = AchievementsInterface->GetAchievementState(MoveTemp(Params));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidUser());
			});

	RunToCompletion();
}