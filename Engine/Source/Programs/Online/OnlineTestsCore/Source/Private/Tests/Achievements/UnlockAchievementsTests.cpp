// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Achievements/QueryAchievementDefinitionsHelper.h"
#include "Helpers/Achievements/QueryAchievementStatesHelper.h"
#include "Helpers/Achievements/UnlockAchievementsHelper.h"
#include "Helpers/Achievements/CheckAchievementStateHelper.h"
#include "Helpers/Stats/UpdateStatsHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/LambdaStep.h"
#include "Online/Achievements.h"
#include "Online/AchievementsErrors.h"
#include "Online/Stats.h"
#include "OnlineCatchHelper.h"

#define ACHIEVEMENTS_TAG "[suite_achievements]"
#define EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG ACHIEVEMENTS_TAG "[unlockachievements]"
#define EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TITLEMANAGED_TAG EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG "[.NULL]"

#define ACHIEVEMENTS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ACHIEVEMENTS_TAG __VA_ARGS__)

ACHIEVEMENTS_TEST_CASE("Unlock Achievements (Invalid User)", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG)
{
	FAccountId AccountId;

	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams;
	UnlockAchievementsHelperParams.OpParams.AchievementIds = { TEXT("test_unlockachievements") };
	UnlockAchievementsHelperParams.ExpectedError = FUnlockAchievementsHelper::ResultType(UE::Online::Errors::InvalidUser());

	GetLoginPipeline({ AccountId })
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Unlock Achievements (Invalid State)", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams;
	UnlockAchievementsHelperParams.OpParams.AchievementIds = { TEXT("test_unlockachievements") };
	UnlockAchievementsHelperParams.ExpectedError = FUnlockAchievementsHelper::ResultType(UE::Online::Errors::InvalidState());
	UnlockAchievementsHelperParams.OpParams.LocalAccountId = AccountId;


	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Unlock Achievements (Invalid Params)", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams;
	UnlockAchievementsHelperParams.ExpectedError = FUnlockAchievementsHelper::ResultType(UE::Online::Errors::InvalidParams());
	UnlockAchievementsHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Unlock Achievements (Not Found)", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams;
	UnlockAchievementsHelperParams.OpParams.AchievementIds = { TEXT("UnknownAchievement") };
	UnlockAchievementsHelperParams.ExpectedError = FUnlockAchievementsHelper::ResultType(UE::Online::Errors::NotFound());
	UnlockAchievementsHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Unlock Achievements (Already Unlocked)", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams1;
	UnlockAchievementsHelperParams1.OpParams.AchievementIds = { TEXT("test_unlockachievements") };
	UnlockAchievementsHelperParams1.OpParams.LocalAccountId = AccountId;
	
	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams2;
	UnlockAchievementsHelperParams2.OpParams.AchievementIds = { TEXT("test_unlockachievements") };
	UnlockAchievementsHelperParams2.ExpectedError = FUnlockAchievementsHelper::ResultType(UE::Online::Errors::Achievements::AlreadyUnlocked());
	UnlockAchievementsHelperParams2.OpParams.LocalAccountId = AccountId;
	
	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams1))
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams2));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Unlock Achievements (Success)", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FQueryAchievementStatesHelper::FHelperParams QueryStatesHelperParams;
	QueryStatesHelperParams.OpParams.LocalAccountId = AccountId;

	FUnlockAchievementsHelper::FHelperParams UnlockAchievementsHelperParams;
	UnlockAchievementsHelperParams.OpParams.AchievementIds = { TEXT("FAKE_ACHIEVEMENT") };
	UnlockAchievementsHelperParams.OpParams.LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceStep<FQueryAchievementStatesHelper>(MoveTemp(QueryStatesHelperParams))
		.EmplaceStep<FUnlockAchievementsHelper>(MoveTemp(UnlockAchievementsHelperParams));

	RunToCompletion();
}

ACHIEVEMENTS_TEST_CASE("Unlock Title-Managed Achievements By Stats", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TITLEMANAGED_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FString AchievementId("FAKE_ACHIEVEMENT");

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FString StatNameKey("Stat_Use_Set");
	FStatValue StatValue(int64(1));

	float AchievementProgress = 0.0f;
	float UpdatedAchievementProgress = 1.0f;

	LoginPipeline
		.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceAsyncLambda([&AccountId, &AchievementId, &AchievementProgress](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) 
			{ 
				CheckAchievementState_Fixture(Promise, Services, AccountId, AchievementId, AchievementProgress);
			})
		.EmplaceAsyncLambda([&AccountId, &StatNameKey, &StatValue](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) 
			{
				FUserStats UsersStats{ AccountId, {{ StatNameKey, StatValue }} };
				TArray<FUserStats> UpdateUsersStats{ UsersStats };

				UpdateStats_Fixture(Promise, Services, AccountId, UpdateUsersStats);
			})
		.EmplaceAsyncLambda([&AccountId, &AchievementId, &UpdatedAchievementProgress](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) 
			{ 
				CheckAchievementState_Fixture(Promise, Services, AccountId, AchievementId, UpdatedAchievementProgress);
			});

	RunToCompletion();
	DestroyCurrentServiceModule();
}

ACHIEVEMENTS_TEST_CASE("Unlock Title-Managed Achievements By Stats With Multiple Conditions", EG_ACHIEVEMENTS_UNLOCKACHIEVEMENTS_TITLEMANAGED_TAG)
{
	FAccountId AccountId;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FString AchievementId("test_unlockachievements");

	FQueryAchievementDefinitionsHelper::FHelperParams QueryDefinitionsHelperParams;
	QueryDefinitionsHelperParams.OpParams.LocalAccountId = AccountId;

	FString FirstStatNameKey("Stat_Use_Largest");
	FString SecondStatNameKey("Stat_Use_Smallest");

	FStatValue FirstStatValue(int64(2));
	FStatValue SecondStatValue(int64(1));
	FStatValue ThirdStatValue(int64(3));

	float AchievementProgress = 0.0f;
	float UpdatedAchievementProgress = 1.0f;

	LoginPipeline
		.EmplaceStep<FQueryAchievementDefinitionsHelper>(MoveTemp(QueryDefinitionsHelperParams))
		.EmplaceAsyncLambda([&AccountId, &AchievementId, &AchievementProgress](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) 
			{ 
				CheckAchievementState_Fixture(Promise, Services, AccountId, AchievementId, AchievementProgress);
			})
		.EmplaceAsyncLambda([&AccountId, &FirstStatNameKey, &FirstStatValue, &SecondStatNameKey, &SecondStatValue](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) 
			{
				FUserStats UserStats{ AccountId, {{ FirstStatNameKey, FirstStatValue}, {SecondStatNameKey, SecondStatValue} } };
				TArray<FUserStats> UpdateUsersStats{ UserStats }; //Partially meet conditions
				UpdateStats_Fixture(Promise, Services, AccountId, UpdateUsersStats);
			})
		.EmplaceAsyncLambda([&AccountId, &AchievementId, &AchievementProgress](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) 
			{
				CheckAchievementState_Fixture(Promise, Services, AccountId, AchievementId, AchievementProgress);
			})
		.EmplaceAsyncLambda([&AccountId, &FirstStatNameKey, &ThirdStatValue, &SecondStatNameKey, &SecondStatValue]
			(const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) 
			{
				FUserStats UpdatedUsersStats{ AccountId, {{ FirstStatNameKey, ThirdStatValue}, {SecondStatNameKey, SecondStatValue } } };
				TArray<FUserStats> UpdateUsersStats{ UpdatedUsersStats}; //Meet all conditions
				UpdateStats_Fixture(Promise, Services, AccountId, UpdateUsersStats);
			})
		.EmplaceAsyncLambda([&AccountId, &AchievementId, &UpdatedAchievementProgress](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) 
			{ 
				CheckAchievementState_Fixture(Promise, Services, AccountId, AchievementId, UpdatedAchievementProgress);
			});

	RunToCompletion();
}