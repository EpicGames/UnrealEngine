// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineAsyncTaskManager.h"

#include <catch2/catch_test_macros.hpp>

#define ASYNCTASKMANAGER_TAG "[AsyncTaskManager]"
#define ASYNCTASKMANAGER_TEST_CASE(x, ...) TEST_CASE(x, ASYNCTASKMANAGER_TAG __VA_ARGS__)

namespace
{

/** Tracks how many instances have been destroyed via a shared counter */
class FMockOnlineAsyncTask : public FOnlineAsyncTask
{
public:
	FMockOnlineAsyncTask(int32& InDestroyCounter)
		: DestroyCounter(InDestroyCounter)
	{
	}

	virtual ~FMockOnlineAsyncTask() override
	{
		++DestroyCounter;
	}

	virtual FString ToString() const override { return TEXT("MockAsyncTask"); }
	virtual bool IsDone() const override { return false; }
	virtual bool WasSuccessful() const override { return false; }

private:
	int32& DestroyCounter;
};

/** FOnlineAsyncItem subclass for testing OutQueue cleanup */
class FMockOnlineAsyncItem : public FOnlineAsyncItem
{
public:
	FMockOnlineAsyncItem(int32& InDestroyCounter)
		: DestroyCounter(InDestroyCounter)
	{
	}

	virtual ~FMockOnlineAsyncItem() override
	{
		++DestroyCounter;
	}

	virtual FString ToString() const override { return TEXT("MockAsyncItem"); }

private:
	int32& DestroyCounter;
};

/** Minimal concrete FOnlineAsyncTaskManager for testing */
class FMockOnlineAsyncTaskManager : public FOnlineAsyncTaskManager
{
public:
	virtual void OnlineTick() override {}
};

} // anonymous namespace

ASYNCTASKMANAGER_TEST_CASE("Destructor cleans up queued serial tasks")
{
	int32 DestroyCount = 0;
	{
		FMockOnlineAsyncTaskManager Manager;
		Manager.AddToInQueue(new FMockOnlineAsyncTask(DestroyCount));
		Manager.AddToInQueue(new FMockOnlineAsyncTask(DestroyCount));
		Manager.AddToInQueue(new FMockOnlineAsyncTask(DestroyCount));
	}
	CHECK(DestroyCount == 3);
}

ASYNCTASKMANAGER_TEST_CASE("Destructor cleans up queued parallel tasks")
{
	int32 DestroyCount = 0;
	{
		FMockOnlineAsyncTaskManager Manager;
		Manager.AddToParallelTasks(new FMockOnlineAsyncTask(DestroyCount));
		Manager.AddToParallelTasks(new FMockOnlineAsyncTask(DestroyCount));
	}
	CHECK(DestroyCount == 2);
}

ASYNCTASKMANAGER_TEST_CASE("Destructor cleans up out queue")
{
	int32 DestroyCount = 0;
	{
		FMockOnlineAsyncTaskManager Manager;
		Manager.AddToOutQueue(new FMockOnlineAsyncItem(DestroyCount));
		Manager.AddToOutQueue(new FMockOnlineAsyncItem(DestroyCount));
	}
	CHECK(DestroyCount == 2);
}

ASYNCTASKMANAGER_TEST_CASE("Destructor cleans up all queues simultaneously")
{
	int32 SerialDestroyCount = 0;
	int32 ParallelDestroyCount = 0;
	int32 OutQueueDestroyCount = 0;
	{
		FMockOnlineAsyncTaskManager Manager;
		Manager.AddToInQueue(new FMockOnlineAsyncTask(SerialDestroyCount));
		Manager.AddToInQueue(new FMockOnlineAsyncTask(SerialDestroyCount));
		Manager.AddToParallelTasks(new FMockOnlineAsyncTask(ParallelDestroyCount));
		Manager.AddToOutQueue(new FMockOnlineAsyncItem(OutQueueDestroyCount));
		Manager.AddToOutQueue(new FMockOnlineAsyncItem(OutQueueDestroyCount));
		Manager.AddToOutQueue(new FMockOnlineAsyncItem(OutQueueDestroyCount));
	}
	CHECK(SerialDestroyCount == 2);
	CHECK(ParallelDestroyCount == 1);
	CHECK(OutQueueDestroyCount == 3);
}

ASYNCTASKMANAGER_TEST_CASE("Destructor handles empty queues gracefully")
{
	// Should not crash when destroying a manager with no tasks
	FMockOnlineAsyncTaskManager Manager;
}
