// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"

#define AUTH_ERROR_TAG "[AUTH]"
#define AUTH_ERROR_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, AUTH_ERROR_TAG __VA_ARGS__)

AUTH_ERROR_TEST_CASE("Basic login test", AUTH_ERROR_TAG)
{
	FAccountId AccountId;
	GetLoginPipeline({ AccountId });
	RunToCompletion();
}
