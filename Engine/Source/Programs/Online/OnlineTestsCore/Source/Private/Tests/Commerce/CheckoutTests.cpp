// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Helpers/Commerce/CheckoutHelper.h"
#include "Helpers/Commerce/GetOffersByIdHelper.h"
#include "Helpers/Commerce/QueryOffersHelper.h"
#include "Misc/ConfigCacheIni.h"

#define COMMERCE_TAG "[suite_commerce]"
#define COMMERCE_CHECKOUT_TAG COMMERCE_TAG "[checkout]"

#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Verify that Checkout returns a fail message if the local user is not logged in", COMMERCE_CHECKOUT_TAG)
{
	FAccountId AccountId;
	TSharedPtr<FPlatformUserId> AccountPlatformUserId = MakeShared<FPlatformUserId>();

	FCommerceCheckout::Params OpCheckoutParams;
	FCheckoutHelper::FHelperParams CheckoutHelperParams;
	CheckoutHelperParams.OpParams = &OpCheckoutParams;
	CheckoutHelperParams.ExpectedError = TOnlineResult<FCommerceCheckout>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	CheckoutHelperParams.OpParams->LocalAccountId = AccountId;

	bool bLogout = false;

	LoginPipeline
		.EmplaceLambda([&AccountId, AccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> UserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ AccountId });
				REQUIRE(UserPlatformUserIdResult.IsOk());
				CHECK(UserPlatformUserIdResult.TryGetOkValue() != nullptr);
				*AccountPlatformUserId = UserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(AccountPlatformUserId))
		.EmplaceStep<FCheckoutHelper>(MoveTemp(CheckoutHelperParams));

	RunToCompletion(bLogout);
}

COMMERCE_TEST_CASE("Verify that Checkout returns a fail message of the given local user ID does not match the actual local user ID", COMMERCE_CHECKOUT_TAG)
{
	FAccountId AccountId;

	FCommerceCheckout::Params OpCheckoutParams;
	FCheckoutHelper::FHelperParams CheckoutHelperParams;
	CheckoutHelperParams.OpParams = &OpCheckoutParams;
	CheckoutHelperParams.ExpectedError = TOnlineResult<FCommerceCheckout>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	CheckoutHelperParams.OpParams->LocalAccountId = FAccountId();

	LoginPipeline
		.EmplaceStep<FCheckoutHelper>(MoveTemp(CheckoutHelperParams));

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that Checkout initiates the checkout process when given an Offers array with one offer", COMMERCE_CHECKOUT_TAG)
{
	SKIP("Test requires web browser to be enabled to run checkout process");

	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 1;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FCommerceCheckout::Params OpCheckoutParams;
	FCheckoutHelper::FHelperParams CheckoutHelperParams;
	CheckoutHelperParams.OpParams = &OpCheckoutParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;

	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	FOfferId OfferId2;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId2"), OfferId2, GEngineIni);
	GetOffersByIdHelperParams.OpParams->OfferIds = { OfferId2 };

	FPurchaseOffer Purchase = { OfferId2, 1};
	CheckoutHelperParams.OpParams->LocalAccountId = AccountId;
	CheckoutHelperParams.OpParams->Offers = { Purchase };

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum)
		.EmplaceStep<FCheckoutHelper>(MoveTemp(CheckoutHelperParams));

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that Checkout initiates the checkout process when given an Offers array with multiple offers")
{
	// TODO
}

COMMERCE_TEST_CASE("Verify that Checkout does not initiate the checkout process when given an empty Offers array")
{
	// TODO
}

COMMERCE_TEST_CASE("Verify that Checkout returns the correct TransactionId after a completed purchase")
{
	// TODO
}

COMMERCE_TEST_CASE("Verify that Checkout does not return a TransactionId if the purchase is incomplete")
{
	// TODO
}
