// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Helpers/Commerce/GetOffersByIdHelper.h"
#include "Helpers/Commerce/QueryOffersHelper.h"
#include "Misc/ConfigCacheIni.h"

#define COMMERCE_TAG "[suite_commerce]"
#define COMMERCE_GETOFFERSBYID_TAG COMMERCE_TAG "[getoffersbyid]"

#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Verify that GetOffersById returns a fail message if the local user is not logged in", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TSharedPtr<FPlatformUserId> AccountPlatformUserId = MakeShared<FPlatformUserId>();

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;
	GetOffersByIdHelperParams.ExpectedError = TOnlineResult<FCommerceGetOffersById>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;

	bool bLogout = false;

	LoginPipeline
		.EmplaceLambda([&AccountId, AccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> UserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ AccountId });
				REQUIRE(UserPlatformUserIdResult.IsOk());
				CHECK(UserPlatformUserIdResult.TryGetOkValue() != nullptr);
				*AccountPlatformUserId = UserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(AccountPlatformUserId))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams));

	RunToCompletion(bLogout);
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns a fail message of the given local user ID does not match the actual local user ID", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;
	GetOffersByIdHelperParams.ExpectedError = TOnlineResult<FCommerceGetOffersById>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	GetOffersByIdHelperParams.OpParams->LocalAccountId = FAccountId();

	LoginPipeline
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams));

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns an empty list when given an empty list of IDs and there are no offers cached", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 0;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	FString CatalogId;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferNamespace0Items"), CatalogId, GEngineIni);
	QueryOffersHelperParams.OpParams->OverrideCatalogNamespace = CatalogId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->OfferIds = {};

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns an empty list when given an empty list of IDs and there are multiple cached offers", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 0;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->OfferIds = {};

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns a fail message when given a populated list of IDs and there are no cached offers", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 0;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;
	GetOffersByIdHelperParams.ExpectedError = TOnlineResult<FCommerceGetOffersById>(Errors::NotFound());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	FString CatalogId;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferNamespace0Items"), CatalogId, GEngineIni);
	QueryOffersHelperParams.OpParams->OverrideCatalogNamespace = CatalogId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	FOfferId OfferId1NotExisting;
	FOfferId OfferId2NotExisting;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId1NotExisting"), OfferId1NotExisting, GEngineIni);
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId2NotExisting"), OfferId2NotExisting, GEngineIni);
	GetOffersByIdHelperParams.OpParams->OfferIds = { OfferId1NotExisting, OfferId2NotExisting };

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns an empty list when given a populated list of IDs and none of them match any cached offers", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 0;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	FOfferId OfferId1NotExisting;
	FOfferId OfferId2NotExisting;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId1NotExisting"), OfferId1NotExisting, GEngineIni);
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId2NotExisting"), OfferId2NotExisting, GEngineIni);
	GetOffersByIdHelperParams.OpParams->OfferIds = { OfferId1NotExisting, OfferId2NotExisting };

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns correct list of one offer when given a populated list of IDs where one ID is cached and another is not", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 1;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	FOfferId OfferId1NotExisting;
	FOfferId OfferId2;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId1NotExisting"), OfferId1NotExisting, GEngineIni);
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId2"), OfferId2, GEngineIni);
	GetOffersByIdHelperParams.OpParams->OfferIds = { OfferId1NotExisting, OfferId2 };

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns the correct list of one offer when given the ID for one cached offer", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 1;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	FOfferId OfferId1;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId1"), OfferId1, GEngineIni);
	GetOffersByIdHelperParams.OpParams->OfferIds = { OfferId1 };

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffersById returns the correct list of offers when given a populated list of multiple cached IDs", COMMERCE_GETOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 2;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffersById::Params OpGetOffersByIdParams;
	FGetOffersByIdHelper::FHelperParams GetOffersByIdHelperParams;
	GetOffersByIdHelperParams.OpParams = &OpGetOffersByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;
	FOfferId OfferId1;
	FOfferId OfferId2;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId1"), OfferId1, GEngineIni);
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferId2"), OfferId2, GEngineIni);
	GetOffersByIdHelperParams.OpParams->OfferIds = { OfferId1, OfferId2 };

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersByIdHelper>(MoveTemp(GetOffersByIdHelperParams), ExpectedOffersNum);

	RunToCompletion();
}