// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Helpers/Commerce/GetOffersHelper.h"
#include "Helpers/Commerce/QueryOffersHelper.h"
#include "Misc/ConfigCacheIni.h"

#define COMMERCE_TAG "[suite_commerce]"
#define COMMERCE_GETOFFERS_TAG COMMERCE_TAG "[getoffers]"

#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Verify that GetOffers returns a fail message if the local user is not logged in", COMMERCE_GETOFFERS_TAG)
{
	FAccountId AccountId;
	TSharedPtr<FPlatformUserId> AccountPlatformUserId = MakeShared<FPlatformUserId>();

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;
	GetOffersHelperParams.ExpectedError = TOnlineResult<FCommerceGetOffers>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	bool bLogout = false;

	LoginPipeline
		.EmplaceLambda([&AccountId, AccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> UserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ AccountId });
				REQUIRE(UserPlatformUserIdResult.IsOk());
				CHECK(UserPlatformUserIdResult.TryGetOkValue() != nullptr);
				*AccountPlatformUserId = UserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(AccountPlatformUserId))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams));

	RunToCompletion(bLogout);
}

COMMERCE_TEST_CASE("Verify that GetOffers returns a fail message of the given local user ID does not match the actual local user ID", COMMERCE_GETOFFERS_TAG)
{
	FAccountId AccountId;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;
	GetOffersHelperParams.ExpectedError = TOnlineResult<FCommerceGetOffers>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	GetOffersHelperParams.OpParams->LocalAccountId = FAccountId();

	LoginPipeline
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams));

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffers returns an empty list if there are no cached offers", COMMERCE_GETOFFERS_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 0;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	FString CatalogId;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferNamespace0Items"), CatalogId, GEngineIni);
	QueryOffersHelperParams.OpParams->OverrideCatalogNamespace = CatalogId;
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffers returns a correct list of one offer if there is only one cached offer", COMMERCE_GETOFFERS_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 1;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	FString CatalogId;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferNamespace1Items"), CatalogId, GEngineIni);
	QueryOffersHelperParams.OpParams->OverrideCatalogNamespace = CatalogId;
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that GetOffers returns the correct list if there are cached offers", COMMERCE_GETOFFERS_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 4;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams), ExpectedOffersNum);

	RunToCompletion();
}