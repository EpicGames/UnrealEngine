// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"

#define COMMERCE_TAG "[suite_commerce]"

#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Verify that OnPurchaseCompleted returns a fail message if the local user is not logged in")
{
	// TODO
}

COMMERCE_TEST_CASE("Verify that OnPurchaseCompleted returns a fail message of the given local user ID does not match the actual local user ID")
{
	// TODO
}

COMMERCE_TEST_CASE("Verify that OnPurchaseCompleted completes without an error")
{
	// TODO
}