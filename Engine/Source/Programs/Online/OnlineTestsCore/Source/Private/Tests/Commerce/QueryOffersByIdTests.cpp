// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Commerce/QueryOffersByIdHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define COMMERCE_TAG "[suite_commerce]"
#define COMMERCE_QUERYOFFERSBYID_TAG COMMERCE_TAG "[queryoffersbyid]"
#define COMMERCE_DISABLED_TAG COMMERCE_TAG "[commercedisabled]"

#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Verify that QueryOffersById returns a fail message if the local user is not logged in", COMMERCE_QUERYOFFERSBYID_TAG)
{
	FAccountId AccountId;
	TSharedPtr<FPlatformUserId> AccountPlatformUserId = MakeShared<FPlatformUserId>();

	FCommerceQueryOffersById::Params OpQueryOffersByIdParams;
	FQueryOffersByIdHelper::FHelperParams QueryOffersByIdHelperParams;
	QueryOffersByIdHelperParams.OpParams = &OpQueryOffersByIdParams;
	QueryOffersByIdHelperParams.ExpectedError = TOnlineResult<FCommerceQueryOffersById>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersByIdHelperParams.OpParams->LocalAccountId = AccountId;

	bool bLogout = false;

	LoginPipeline
		.EmplaceLambda([&AccountId, AccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> UserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ AccountId });
				REQUIRE(UserPlatformUserIdResult.IsOk());
				CHECK(UserPlatformUserIdResult.TryGetOkValue() != nullptr);
				*AccountPlatformUserId = UserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(AccountPlatformUserId))
		.EmplaceStep<FQueryOffersByIdHelper>(MoveTemp(QueryOffersByIdHelperParams));

	RunToCompletion(bLogout);
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns a fail message of the given local user ID does not match the actual local user ID", COMMERCE_QUERYOFFERSBYID_TAG)
{
	FAccountId AccountId;

	FCommerceQueryOffersById::Params OpQueryOffersByIdParams;
	FQueryOffersByIdHelper::FHelperParams QueryOffersByIdHelperParams;
	QueryOffersByIdHelperParams.OpParams = &OpQueryOffersByIdParams;
	QueryOffersByIdHelperParams.ExpectedError = TOnlineResult<FCommerceQueryOffersById>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersByIdHelperParams.OpParams->LocalAccountId = FAccountId();

	LoginPipeline
		.EmplaceStep<FQueryOffersByIdHelper>(MoveTemp(QueryOffersByIdHelperParams));

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns an empty list when given an empty list of IDs and there are no existing offers", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns an empty list when given an empty list of IDs and there are multiple existing offers", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns a fail message when given a populated list of IDs and there are no existing offers", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns a fail message when given a populated list of IDs and none of them match existing offers", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns a fail message when given a populated list of IDs where one ID exists and another does not", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns the correct list of one offer when given the ID for one existing offer", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}

COMMERCE_TEST_CASE("Verify that QueryOffersById returns the correct list of offers when given a populated list of multiple existing IDs", COMMERCE_DISABLED_TAG)
{
	// EOS QueryOffers interface doesn't support anything ID-specific, skipping the test
}