// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreGlobals.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Helpers/Commerce/GetOffersHelper.h"
#include "Helpers/Commerce/QueryOffersHelper.h"
#include "Misc/ConfigCacheIni.h"

#define COMMERCE_TAG "[suite_commerce]"
#define COMMERCE_QUERYOFFERS_TAG COMMERCE_TAG "[queryoffers]"

#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Basic compile test")
{
	TSharedPtr<ICommerce> Commerce;
	CHECK(!Commerce.IsValid());
}

COMMERCE_TEST_CASE("Verify that QueryOffers returns a fail message if the local user is not logged in", COMMERCE_QUERYOFFERS_TAG)
{
	FAccountId AccountId;
	TSharedPtr<FPlatformUserId> AccountPlatformUserId = MakeShared<FPlatformUserId>();

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;
	QueryOffersHelperParams.ExpectedError = TOnlineResult<FCommerceQueryOffers>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;

	bool bLogout = false;

	LoginPipeline
		.EmplaceLambda([&AccountId, AccountPlatformUserId](const IOnlineServicesPtr& OnlineServices)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineServices->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> UserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ AccountId });
				REQUIRE(UserPlatformUserIdResult.IsOk());
				CHECK(UserPlatformUserIdResult.TryGetOkValue() != nullptr);
				*AccountPlatformUserId = UserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(AccountPlatformUserId))
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams));

	RunToCompletion(bLogout);
}

COMMERCE_TEST_CASE("Verify that QueryOffers returns a fail message of the given local user ID does not match the actual local user ID", COMMERCE_QUERYOFFERS_TAG)
{
	FAccountId AccountId;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;
	QueryOffersHelperParams.ExpectedError = TOnlineResult<FCommerceQueryOffers>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = FAccountId();

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams));

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that QueryOffers caches an empty list if there are no offers", COMMERCE_QUERYOFFERS_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 0;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	FString CatalogId;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferNamespace0Items"), CatalogId, GEngineIni);
	QueryOffersHelperParams.OpParams->OverrideCatalogNamespace = CatalogId;
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that QueryOffers caches a list of one offer if there is only one existing offer", COMMERCE_QUERYOFFERS_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 1;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	FString CatalogId;
	GConfig->GetString(TEXT("EOSSDK.Platform.OnlineTests"), TEXT("OfferNamespace1Items"), CatalogId, GEngineIni);
	QueryOffersHelperParams.OpParams->OverrideCatalogNamespace = CatalogId;
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams), ExpectedOffersNum);

	RunToCompletion();
}

COMMERCE_TEST_CASE("Verify that QueryOffers caches the list of all offers if there are multiple existing offers", COMMERCE_QUERYOFFERS_TAG)
{
	FAccountId AccountId;
	TOptional<uint32_t> ExpectedOffersNum = 4;

	FCommerceQueryOffers::Params OpQueryOffersParams;
	FQueryOffersHelper::FHelperParams QueryOffersHelperParams;
	QueryOffersHelperParams.OpParams = &OpQueryOffersParams;

	FCommerceGetOffers::Params OpGetOffersParams;
	FGetOffersHelper::FHelperParams GetOffersHelperParams;
	GetOffersHelperParams.OpParams = &OpGetOffersParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	QueryOffersHelperParams.OpParams->LocalAccountId = AccountId;
	GetOffersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryOffersHelper>(MoveTemp(QueryOffersHelperParams))
		.EmplaceStep<FGetOffersHelper>(MoveTemp(GetOffersHelperParams), ExpectedOffersNum);

	RunToCompletion();
}
