// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"

#define COMMERCE_TAG "[suite_commerce]"

#define COMMERCE_DISABLED_TAG COMMERCE_TAG "[commercedisabled]"
#define COMMERCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, COMMERCE_TAG __VA_ARGS__)

COMMERCE_TEST_CASE("Verify that ShowStoreUI returns a fail message if the local user is not logged in", COMMERCE_DISABLED_TAG)
{
	// TODO
	// The ShowStoreUI EOS interface is not implemented
}

COMMERCE_TEST_CASE("Verify that ShowStoreUI returns a fail message of the given local user ID does not match the actual local user ID", COMMERCE_DISABLED_TAG)
{
	// TODO
	// The ShowStoreUI EOS interface is not implemented
}

COMMERCE_TEST_CASE("Verify that ShowStoreUI displays platform store UI", COMMERCE_DISABLED_TAG) // May not be possible to automate currently
{
	// TODO
	// The ShowStoreUI EOS interface is not implemented
}