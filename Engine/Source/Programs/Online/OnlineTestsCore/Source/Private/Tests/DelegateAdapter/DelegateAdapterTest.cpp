// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"
#include "Online/DelegateAdapter.h"
#include "Online/MulticastAdapter.h"

#define DA_ERROR_TAG "[DelegateAdapter]"
#define DA_ERROR_TEST_CASE(x, ...) TEST_CASE(x, DA_ERROR_TAG __VA_ARGS__)

DECLARE_MULTICAST_DELEGATE_OneParam(FDelegateTest, int /*Num*/);
typedef FDelegateTest::FDelegate FDelegateTestDelegate;

class FDelegateTestHelper
	: public TSharedFromThis<FDelegateTestHelper>
{
public:
	virtual ~FDelegateTestHelper() {}

	DEFINE_ONLINE_DELEGATE_ONE_PARAM(DelegateTest, int32 /*LocalUserNum*/);

	DECLARE_DELEGATE_OneParam(FOnTestDelegate, int32);

	void ExecuteDelegate(const FOnTestDelegate& Delegate, int32 Value)
	{
		Delegate.ExecuteIfBound(Value);
	}
};


DA_ERROR_TEST_CASE("Basic multicast delegate test")
{
	OnlineAutoReg::CheckRunningTestSkipOnTags();

	TSharedRef<FDelegateTestHelper> Helper = MakeShared<FDelegateTestHelper>();
	bool bDidMulticastExecute = false;

	auto Da = UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&bDidMulticastExecute](int32 UserNum) mutable
	{
		CHECK(UserNum == 50);
		bDidMulticastExecute = true;
	});

	Helper->TriggerDelegateTestDelegates(50);
	CHECK(bDidMulticastExecute);
}

DA_ERROR_TEST_CASE("Multicast- ensure unbind is working properly")
{

	OnlineAutoReg::CheckRunningTestSkipOnTags();

	TSharedPtr<FDelegateTestHelper> Helper = MakeShared<FDelegateTestHelper>();
	TWeakPtr<FDelegateTestHelper> HelperWeak = TWeakPtr<FDelegateTestHelper>(Helper);
	bool bDidMulticastExecute = false;
	int exec1 = 0;
	int exec2 = 0;
	int exec3 = 0;
	int exec4 = 0;
	int exec5 = 0;

	auto Da1 =  UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&exec1](int32 UserNum) mutable
	{
		exec1++;
	})->AsWeak();
	Helper->TriggerDelegateTestDelegates(50);
	
	auto Da2 = UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&exec2](int32 UserNum) mutable
	{
		exec2++;
	})->AsWeak();
	Helper->TriggerDelegateTestDelegates(50);

	auto Da3 = UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&exec3](int32 UserNum) mutable
	{
		exec3++;
	})->AsWeak();
	Helper->TriggerDelegateTestDelegates(50);

	auto Da4 = UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&exec4](int32 UserNum) mutable
	{
		exec4++;
	})->AsWeak();
	Helper->TriggerDelegateTestDelegates(50);

	auto Da5 = UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&exec5](int32 UserNum) mutable
	{
		exec5++;
	})->AsWeak();
	Helper->TriggerDelegateTestDelegates(50);


	CHECK(exec1 == 1);
	CHECK(exec2 == 1);
	CHECK(exec3 == 1);
	CHECK(exec4 == 1);
	CHECK(exec5 == 1);
	CHECK(!Da1.IsValid());
	CHECK(!Da2.IsValid());
	CHECK(!Da3.IsValid());
	CHECK(!Da4.IsValid());
	CHECK(!Da5.IsValid());
	Helper.Reset();
	CHECK(!HelperWeak.IsValid());
}

DA_ERROR_TEST_CASE("Multicast- ensure unique ptrs are preserved")
{
	OnlineAutoReg::CheckRunningTestSkipOnTags();

	TSharedRef<FDelegateTestHelper> Helper = MakeShared<FDelegateTestHelper>();
	bool bDidMulticastExecute = false;
	TUniquePtr<int> UniquePtr = MakeUnique<int>(1213141);

	auto Da = UE::Online::MakeMulticastAdapter(Helper, Helper->DelegateTestDelegates,
	[&bDidMulticastExecute, UniquePtr = MoveTemp(UniquePtr)](int32 UserNum) mutable
	{
		CHECK(UserNum == 50);
		bDidMulticastExecute = true;
		CHECK(*UniquePtr == 1213141);
	});

	Helper->TriggerDelegateTestDelegates(50);
	CHECK(bDidMulticastExecute);
}

DA_ERROR_TEST_CASE("Basic single delegate test")
{
	OnlineAutoReg::CheckRunningTestSkipOnTags();

	TSharedPtr<FDelegateTestHelper> Helper = MakeShared<FDelegateTestHelper>();
	bool bDidMulticastExecute = false;

	auto Da = MakeDelegateAdapter(Helper, [&bDidMulticastExecute](int32 Value) mutable{
		CHECK(Value == 50);
		bDidMulticastExecute = true;
	});

	Helper->ExecuteDelegate(*Da, 50);

	CHECK(bDidMulticastExecute);
}
