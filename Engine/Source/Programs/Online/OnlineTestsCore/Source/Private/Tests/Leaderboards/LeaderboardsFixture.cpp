// Copyright Epic Games, Inc. All Rights Reserved.

#include "LeaderboardsFixture.h"


const UE::Online::FLeaderboardEntry* LeaderboardEntryOf(const TArray<UE::Online::FLeaderboardEntry>& LeaderboardEntries, const FAccountId& AccountId)
{
	return LeaderboardEntries.FindByPredicate([&AccountId](const UE::Online::FLeaderboardEntry& LeaderboardEntry) { return LeaderboardEntry.AccountId == AccountId; });
}

FTestPipeline&& LeaderboardsFixture_NUsers(FTestPipeline&& Pipeline, TArray<FAccountId*>& AccountIds, const uint32& NumUsersToLogin)
{
	return Pipeline.EmplaceAsyncLambda([&](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
		TArray<FUserStats> UpdateUsersStats;
		UpdateUsersStats.Reserve(NumUsersToLogin);
		uint32 Counter = 0;
		for (const FAccountId* Account : AccountIds) {
			UpdateUsersStats.Push({ *Account, {{ "Stat_Use_Set", FStatValue((int64)(NumUsersToLogin - Counter) + 2)}, { "Stat_Use_Smallest", FStatValue((int64)(NumUsersToLogin - Counter) + 5)}} });
			Counter++;	
		}
		Services->GetStatsInterface()->ResetStats({ *AccountIds[0] });
			// Write stats which triggers WriteLeaderboards underneath 
			UpdateStats_Fixture(Promise, Services, *AccountIds[0], UpdateUsersStats);
		});
}

void CheckEntries_2Users_Stat_Use_Set(const TArray<UE::Online::FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB)
{
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Rank, 0);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Score, 4);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Rank, 1);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Score, 3);
}

void CheckEntries_2UsersRanks_Stat_Use_Set(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB)
{
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Rank, 3);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Score, 4);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Rank, 4);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Score, 3);
}

void CheckEntries_2UsersRanks_Stat_Use_Set_Index_Moved_Up(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB)
{
	CHECK_EQUAL(LeaderboardEntries.Num(), 2);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Rank, 2);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Rank, 3);
}

void CheckEntries_2Users_Stat_Use_Smallest(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB)
{
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Rank, 0);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdB)->Score, 6);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Rank, 1);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Score, 7);
}

void CheckEntries_1User_Stat_Use_Smallest(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA)
{
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Rank, 1);
	CHECK_EQUAL(LeaderboardEntryOf(LeaderboardEntries, AccountIdA)->Score, 7);
}