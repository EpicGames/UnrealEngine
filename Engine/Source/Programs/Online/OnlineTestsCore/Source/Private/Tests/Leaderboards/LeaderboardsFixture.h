// Copyright Epic Games, Inc. All Rights Reserved.

#include <catch2/catch_test_macros.hpp>

#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/Stats/UpdateStatsHelper.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"
#include "Online/Stats.h"
#include "Online/Leaderboards.h"

const UE::Online::FLeaderboardEntry* LeaderboardEntryOf(const TArray<UE::Online::FLeaderboardEntry>& LeaderboardEntries, const FAccountId& AccountId);

FTestPipeline&& LeaderboardsFixture_NUsers(FTestPipeline&& Pipeline, TArray<FAccountId*>& AccountIds, const uint32& NumUsersToLogin);

void CheckEntries_2Users_Stat_Use_Set(const TArray<UE::Online::FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB);
void CheckEntries_2UsersRanks_Stat_Use_Set(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB);
void CheckEntries_2UsersRanks_Stat_Use_Set_Index_Moved_Up(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB);

void CheckEntries_1User_Stat_Use_Smallest(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA);
void CheckEntries_2Users_Stat_Use_Smallest(const TArray<FLeaderboardEntry>& LeaderboardEntries, FAccountId& AccountIdA, FAccountId& AccountIdB);
