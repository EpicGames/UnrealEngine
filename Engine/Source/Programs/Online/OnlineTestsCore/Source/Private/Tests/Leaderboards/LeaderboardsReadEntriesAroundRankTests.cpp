// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadEntriesAroundRankFixture.h"
#include "LeaderboardsFixture.h"

#define LEADERBOARDS_TAGS "[Leaderboards]"
#define LEADERBOARDS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, LEADERBOARDS_TAGS __VA_ARGS__)


// ReadEntriesAroundRank tests

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundRank succeeds and returns entries for the given board", "[MultiAccount]")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountIdA, AccountIdB;
	TArray< FAccountId*> AccountIds = { &AccountIdA, &AccountIdB };
	uint32 NumUsersToLogin = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountIdA, AccountIdB })), AccountIds, NumUsersToLogin)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundRank_2UsersFixture(Promise, Services, "Stat_Use_Set", AccountIdA, 0, 2, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Set(LeaderboardEntries, AccountIdA, AccountIdB); })
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundRank_2UsersFixture(Promise, Services, "Stat_Use_Smallest", AccountIdA, 0, 2, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Smallest(LeaderboardEntries, AccountIdA, AccountIdB); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundRank returns a fail message if the local user is not logged in", "[MultiAccount]")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountId;
	FAccountId LocalUserInvalid;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUserInvalid };

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountId })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundRankFixture(Promise, Services, "Stat_Use_Set", LocalUserInvalid, 0, 2, UE::Online::Errors::InvalidUser(), LeaderboardEntries); });

	RunToCompletion();
}


LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundRank returns a fail message if given a Rank past the last existing rank")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId };

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountId })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundRankFixture(Promise, Services, "Stat_Use_Set", AccountId, 1, 2, UE::Online::Errors::InvalidParams(), LeaderboardEntries); });

	RunToCompletion();
}


LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundRank returns all valid entries if the given Limit exceeds the number of existing entries after the given Rank")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountIdA, AccountIdB;
	TArray< FAccountId*> AccountIds = { &AccountIdA, &AccountIdB };
	uint32 rank = 0;
	uint32 limit = 3;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountIdA, AccountIdB })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundRank_2UsersFixture(Promise, Services, "Stat_Use_Set", AccountIdA, rank, limit, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Set(LeaderboardEntries, AccountIdA, AccountIdB); });

	RunToCompletion();
}


LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundRank returns fail message if given a Limit of 0")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId };

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountId })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundRankFixture(Promise, Services, "Stat_Use_Set", AccountId, 3, 0, UE::Online::Errors::InvalidParams(), LeaderboardEntries); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundRank returns a fail message if given an invalid BoardName")
{
	// TODO
}