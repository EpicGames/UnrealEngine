// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadEntriesAroundUserFixture.h"
#include "LeaderboardsFixture.h"

#define LEADERBOARDS_TAGS "[Leaderboards]"
#define LEADERBOARDS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, LEADERBOARDS_TAGS __VA_ARGS__)


// ReadEntriesAroundUser tests

LEADERBOARDS_TEST_CASE("Verify that ReadEntriesAroundUser succeeds and returns entries for the given board", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TArray< FAccountId*> AccountIds = { &AccountIdA, &AccountIdB };
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	int32 Offset = 0;
	uint32 Limit = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountIdA, AccountIdB })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUser_2UsersFixture(Promise, Services, "Stat_Use_Set", AccountIdB, AccountIdA, Offset, Limit, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Set(LeaderboardEntries, AccountIdA, AccountIdB); })
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUser_2UsersFixture(Promise, Services, "Stat_Use_Smallest", AccountIdA, AccountIdB, -1, 2, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Smallest(LeaderboardEntries, AccountIdA, AccountIdB); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns a fail message if the local user is not logged in", "[MultiAccount]")
{
	FAccountId LocalUserInvalid;
	FAccountId AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUserInvalid };
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	int32 offset = 0;
	uint32 limit = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountId })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUserFixture(Promise, Services, "Stat_Use_Set", LocalUserInvalid, AccountId, offset, limit, UE::Online::Errors::InvalidUser(), LeaderboardEntries); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns a fail message if the given UserId is invalid", "[MultiAccount]")
{
	FAccountId LocalUser;
	FAccountId AccountIdInvalid;
	TArray< FAccountId*> AccountIds = { &LocalUser, &AccountIdInvalid };
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	int32 offset = 0;
	uint32 limit = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUserFixture(Promise, Services, "Stat_Use_Set", LocalUser, AccountIdInvalid, offset, limit, UE::Online::Errors::InvalidUser(), LeaderboardEntries); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAround User returns entries centered around the correct position when the given Offset is positive", "[MultiAccount]")
{
	FAccountId LocalUser, OtherPlayerB, UserPlayer, OtherPlayerC, OtherPlayerD;
	TArray< FAccountId*> AccountIds = { &LocalUser, &OtherPlayerB, &UserPlayer, &OtherPlayerC, &OtherPlayerD };
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	int32 offset = 1;
	uint32 limit = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser, OtherPlayerB, UserPlayer, OtherPlayerC, OtherPlayerD })), AccountIds, 5)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUser_5UsersFixture(Promise, Services, "Stat_Use_Set", { LocalUser, OtherPlayerB, UserPlayer, OtherPlayerC, OtherPlayerD }, offset, limit, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2UsersRanks_Stat_Use_Set(LeaderboardEntries, OtherPlayerC, OtherPlayerD); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns a number of entries that matches the given Limit", "[MultiAccount]")
{

	FAccountId LocalUser, AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUser };
	int32 offset = 0;
	uint32 limit = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser, AccountId })), AccountIds, 2)
		.EmplaceAsyncLambda([&](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) 
		{
			Services->GetLeaderboardsInterface()->ReadEntriesAroundUser({ LocalUser, AccountId, offset, limit, "Stat_Use_Set" })
			.OnComplete([Promise, limit](const TOnlineResult<UE::Online::FReadEntriesAroundUser>& Result) mutable 
				{
					CHECK_EQUAL(Result.GetOkValue().Entries.Num(), limit);
					Promise->SetValue(true);
				});
			});

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns expected valid entries if the given Limit exceeds the number of existing entries before the center entry", "[MultiAccount]")
{
	FAccountId LocalUser, AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUser };
	int32 offset = -1;
	uint32 limit = 4;
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountId, LocalUser })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUser_2UsersFixture(Promise, Services, "Stat_Use_Set", LocalUser, AccountId, offset, limit, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Set(LeaderboardEntries, AccountId, LocalUser); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns expected valid entries if the given Limit exceeds the number of existing entries after the center entry", "[MultiAccount]")
{
	FAccountId LocalUser, AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUser };
	int32 offset = 0;
	uint32 limit = 8;
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser, AccountId })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUser_2UsersFixture(Promise, Services, "Stat_Use_Smallest", LocalUser, AccountId, offset, limit, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_1User_Stat_Use_Smallest(LeaderboardEntries, AccountId); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns error message if given a Limit of 0", "[MultiAccount]")
{
	FAccountId LocalUser, AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUser };
	int32 offset = 0;
	uint32 limit = 0;
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser, AccountId })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) {ReadEntriesAroundUserFixture(Promise, Services, "Stat_Use_Set", LocalUser, AccountId, offset, limit, UE::Online::Errors::InvalidParams(), LeaderboardEntries); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns only one entry if given Offset of 0 and  Limit of 1", "[MultiAccount]")
{
	FAccountId LocalUser, AccountId;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUser };
	int32 offset = 0;
	uint32 limit = 1;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser, AccountId })), AccountIds, 2)
		.EmplaceAsyncLambda([&](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) 
			{
			Services->GetLeaderboardsInterface()->ReadEntriesAroundUser({ LocalUser, AccountId, offset, limit, "Stat_Use_Set" })
			.OnComplete([Promise, limit](const TOnlineResult<UE::Online::FReadEntriesAroundUser>& Result) mutable 
				{
					CHECK_EQUAL(Result.GetOkValue().Entries.Num(), limit);
					Promise->SetValue(true);
				});
			});

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns expected valid entries if  given the Offset=-1 and Limit=2", "[MultiAccount]")
{
	FAccountId OtherPlayerA, OtherPlayerB, UserPlayer, OtherPlayerC, OtherPlayerD;
	TArray< FAccountId*> AccountIds = { &OtherPlayerA, &OtherPlayerB, &UserPlayer, &OtherPlayerC, &OtherPlayerD };
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	int32 offset = 0;
	uint32 limit = 2;

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ OtherPlayerA, OtherPlayerB, UserPlayer, OtherPlayerC, OtherPlayerD })), AccountIds, 5)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesAroundUser_5UsersFixture(Promise, Services, "Stat_Use_Set", { OtherPlayerA, OtherPlayerB, UserPlayer, OtherPlayerC, OtherPlayerD }, offset, limit, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2UsersRanks_Stat_Use_Set_Index_Moved_Up(LeaderboardEntries, UserPlayer, OtherPlayerC); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesAroundUser returns a fail message if given an invalid BoardName")
{
	// TODO
}

