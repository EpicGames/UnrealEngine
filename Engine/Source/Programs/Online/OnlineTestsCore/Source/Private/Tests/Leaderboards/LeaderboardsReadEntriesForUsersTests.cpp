// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadEntiresForUsersFixture.h"
#include "LeaderboardsFixture.h"

#define LEADERBOARDS_TAGS "[Leaderboards]"
#define LEADERBOARDS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, LEADERBOARDS_TAGS __VA_ARGS__)


// ReadEntriesForUsers tests

LEADERBOARDS_TEST_CASE("ReadEntriesForUsers succeed", "[MultiAccount]")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountIdA, AccountIdB;
	TArray< FAccountId*> AccountIds = { &AccountIdA, &AccountIdB };

	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountIdA, AccountIdB })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesForUsers_2UsersFixture(Promise, Services, "Stat_Use_Set", AccountIdA, AccountIdB, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Set(LeaderboardEntries, AccountIdA, AccountIdB); })
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesForUsers_2UsersFixture(Promise, Services, "Stat_Use_Smallest", AccountIdA, AccountIdB, LeaderboardEntries); })
		.EmplaceLambda([&](const IOnlineServicesPtr& Services) { CheckEntries_2Users_Stat_Use_Smallest(LeaderboardEntries, AccountIdA, AccountIdB); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesForUsers returns a fail message if the local user is not logged in", "[MultiAccount]")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountId;	
	FAccountId LocalUserInvalid;
	TArray< FAccountId*> AccountIds = { &AccountId, &LocalUserInvalid };
	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ AccountId })), AccountIds, 2)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesForUsersFixture(Promise, Services, "Stat_Use_Set", { LocalUserInvalid, AccountId }, UE::Online::Errors::InvalidUser(), LeaderboardEntries); });

	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesForUsers returns an empty array of entries if the given UserIds array is empty")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId LocalUser;
	TArray< FAccountId*> AccountIds = { &LocalUser };
	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) {
		Services->GetLeaderboardsInterface()->ReadEntriesForUsers({ LocalUser, {}, "Stat_Use_Set" })
		.OnComplete([Promise, &LeaderboardEntries](const TOnlineResult<UE::Online::FReadEntriesForUsers>& Result) mutable
			{
				CHECK(LeaderboardEntries.IsEmpty());
				Promise->SetValue(true);
			});
		});
	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntiresForUsers returns a fail message if given a UserIds array that contains one user ID which is invalid", "[MultiAccount]")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountIdInvalid;
	FAccountId LocalUser;
	TArray< FAccountId*> AccountIds = { &LocalUser, &AccountIdInvalid };
	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser })), AccountIds, 1)
		.EmplaceAsyncLambda([&](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) 
		{
			Services->GetLeaderboardsInterface()->ReadEntriesForUsers({ LocalUser, {AccountIdInvalid}, "Stat_Use_Set" })
			.OnComplete([Promise, &LeaderboardEntries](const TOnlineResult<UE::Online::FReadEntriesForUsers>& Result) mutable 
			{
				if (Result.IsError())
				{
					CHECK(Result.GetErrorValue() == UE::Online::Errors::InvalidUser());
					CHECK(LeaderboardEntries.IsEmpty());
				}
				else
				{
					CHECK(LeaderboardEntries.IsEmpty());
				}
				Promise->SetValue(true);
			});
		});
	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesForUsers returns a fail message if given a UserIds array that contains one valid user ID and one invalid user ID", "[MultiAccount]")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId AccountIdInvalid;
	FAccountId LocalUser;
	TArray< FAccountId*> AccountIds = { &LocalUser, &AccountIdInvalid };
	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser })), AccountIds, 1)
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services) { ReadEntriesForUsersFixture(Promise, Services, "Stat_Use_Set", { LocalUser, AccountIdInvalid }, UE::Online::Errors::InvalidUser(), LeaderboardEntries); });
	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesForUsers returns an Entries array with only the expected entry if given a UserIds array with one ID")
{
	TArray<UE::Online::FLeaderboardEntry> LeaderboardEntries;
	FAccountId LocalUser;
	TArray< FAccountId*> AccountIds = { &LocalUser };
	LeaderboardsFixture_NUsers(MoveTemp(GetLoginPipeline({ LocalUser })), AccountIds, 1)
		.EmplaceAsyncLambda([&](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) 
		{
			Services->GetLeaderboardsInterface()->ReadEntriesForUsers({ LocalUser, {LocalUser}, "Stat_Use_Set" })
			.OnComplete([Promise, LeaderboardEntries, LocalUser](const TOnlineResult<UE::Online::FReadEntriesForUsers>& Result) mutable 
			{
				const bool bIsOk = Result.IsOk();
				if (bIsOk)
				{
					CHECK_EQUAL(Result.GetOkValue().Entries.Num(), 1);
					CHECK_EQUAL(LeaderboardEntryOf(Result.GetOkValue().Entries, LocalUser)->Rank, 1);
					CHECK_EQUAL(LeaderboardEntryOf(Result.GetOkValue().Entries, LocalUser)->Score, 3);
				}
				Promise->SetValue(true);
			});
		});
	RunToCompletion();
}

LEADERBOARDS_TEST_CASE("Verify ReadEntriesForUsers returns a fail message if given an invalid BoardName")
{
	// TODO
}

