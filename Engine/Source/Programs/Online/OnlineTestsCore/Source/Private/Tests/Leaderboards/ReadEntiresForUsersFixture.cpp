// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadEntiresForUsersFixture.h"


void ReadEntriesForUsers_2UsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountIdA, const FAccountId& AccountIdB, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesForUsers({ AccountIdA, {AccountIdA, AccountIdB}, BoardName }).OnComplete([Promise, &OutLeaderboardEntries](const TOnlineResult<UE::Online::FReadEntriesForUsers>& Result) mutable
		{
			CHECK_OP(Result);
			Promise->SetValue(true);
			OutLeaderboardEntries = Result.GetOkValue().Entries;
		});
}

void ReadEntriesForUsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const TArray<FAccountId>& AccountIds, const UE::Online::FOnlineError& InError, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesForUsers({ AccountIds[0], AccountIds, BoardName }).OnComplete([Promise, &OutLeaderboardEntries, InError](const TOnlineResult<UE::Online::FReadEntriesForUsers>& Result) mutable
		{
			//CHECK_OP_EQ(Result, InError);
			if (Result.IsError())
			{
				CHECK(Result.GetErrorValue() == InError);
				CHECK(OutLeaderboardEntries.IsEmpty());
			}
			else
			{
				CHECK(OutLeaderboardEntries.IsEmpty());
			}
				Promise->SetValue(true);
		});
}