// Copyright Epic Games, Inc. All Rights Reserved.

#include <catch2/catch_test_macros.hpp>

#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/Stats/UpdateStatsHelper.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"
#include "Online/Stats.h"
#include "Online/Leaderboards.h"


void ReadEntriesForUsers_2UsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountIdA, const FAccountId& AccountIdB, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries);
void ReadEntriesForUsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const TArray<FAccountId>& AccountIds, const UE::Online::FOnlineError& InError, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries);

