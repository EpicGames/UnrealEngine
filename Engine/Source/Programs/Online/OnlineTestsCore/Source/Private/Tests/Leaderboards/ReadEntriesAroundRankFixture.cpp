// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadEntriesAroundRankFixture.h"

void ReadEntriesAroundRank_2UsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountIdA, uint32 Rank, uint32 Limit, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesAroundRank({ AccountIdA, Rank, Limit, BoardName }).OnComplete([Promise, &OutLeaderboardEntries](const TOnlineResult<UE::Online::FReadEntriesAroundRank>& Result) mutable
		{
			CHECK_OP(Result);
			Promise->SetValue(true);
			OutLeaderboardEntries = Result.GetOkValue().Entries;
		});
}

void ReadEntriesAroundRankFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountId, uint32 Rank, uint32 Limit, const UE::Online::FOnlineError& InError, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesAroundRank({ AccountId, Rank, Limit, BoardName }).OnComplete([Promise, &OutLeaderboardEntries, InError](const TOnlineResult<UE::Online::FReadEntriesAroundRank>& Result) mutable
		{
			if (Result.IsError())
			{
				CHECK(Result.GetErrorValue() == InError);
				CHECK(OutLeaderboardEntries.IsEmpty());
			}
			else
			{
				CHECK(OutLeaderboardEntries.IsEmpty());
			}
	Promise->SetValue(true);
		});
}