// Copyright Epic Games, Inc. All Rights Reserved.

#include <catch2/catch_test_macros.hpp>

#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/Stats/UpdateStatsHelper.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"
#include "Online/Stats.h"
#include "Online/Leaderboards.h"


void ReadEntriesAroundRank_2UsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountIdA, uint32 Rank, uint32 Limit, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries);
void ReadEntriesAroundRankFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountId, uint32 Rank, uint32 Limit, const UE::Online::FOnlineError& InError, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries);
