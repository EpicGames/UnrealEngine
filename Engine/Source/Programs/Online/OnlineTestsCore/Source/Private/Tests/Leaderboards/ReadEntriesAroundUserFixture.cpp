// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReadEntriesAroundUserFixture.h"

void ReadEntriesAroundUser_2UsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountIdA, const FAccountId& AccountIdB, int32 Offset, uint32 Limit, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesAroundUser({ AccountIdA, AccountIdB, Offset, Limit, BoardName }).OnComplete([Promise, &OutLeaderboardEntries](const TOnlineResult<UE::Online::FReadEntriesAroundUser>& Result) mutable
		{
			CHECK_OP(Result);
			Promise->SetValue(true);
			OutLeaderboardEntries = Result.GetOkValue().Entries;
		});
}

void ReadEntriesAroundUser_5UsersFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const TArray<FAccountId>& AccountIds, int32 Offset, uint32 Limit, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesAroundUser({ AccountIds[0], AccountIds[2], Offset, Limit, BoardName }).OnComplete([Promise, &OutLeaderboardEntries](const TOnlineResult<UE::Online::FReadEntriesAroundUser>& Result) mutable
		{
			CHECK(Result.IsOk());
			Promise->SetValue(true);
			OutLeaderboardEntries = Result.GetOkValue().Entries;
		});
}

void ReadEntriesAroundUserFixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FString& BoardName, const FAccountId& AccountIdA, const FAccountId& AccountIdB, int32 Offset, uint32 Limit, const UE::Online::FOnlineError& InError, TArray<UE::Online::FLeaderboardEntry>& OutLeaderboardEntries)
{
	Services->GetLeaderboardsInterface()->ReadEntriesAroundUser({ AccountIdA, AccountIdB, Offset, Limit, BoardName }).OnComplete([Promise, &OutLeaderboardEntries, InError](const TOnlineResult<UE::Online::FReadEntriesAroundUser>& Result) mutable
		{
			if (Result.IsError())
			{
				CHECK(Result.GetErrorValue() == InError);
				CHECK(OutLeaderboardEntries.IsEmpty());
			}
			else
			{
				CHECK(OutLeaderboardEntries.IsEmpty());
			}
			Promise->SetValue(true);
		});
}