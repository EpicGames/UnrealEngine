// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Lobby/LobbyCreateHelper.h"
#include "Helpers/Lobby/LobbyJoinHelper.h"

#define LOBBY_CREATE_INVITE_TAGS "[Lobby]"
#define LOBBY_CREATE_INVITE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, LOBBY_CREATE_INVITE_TAGS __VA_ARGS__)

LOBBY_CREATE_INVITE_TEST_CASE("Basic lobby create and join test")
{
	int32 TestAccountIndex = 1;
	FAccountId AccountId;

	UE::Online::FCreateLobby::Params LobbyCreateParams;
	LobbyCreateParams.LocalName = TEXT("TestLobby");
	LobbyCreateParams.SchemaId = TEXT("test");
	LobbyCreateParams.MaxMembers = 2;
	LobbyCreateParams.JoinPolicy = ELobbyJoinPolicy::PublicAdvertised;
	LobbyCreateParams.Attributes = { { TEXT("LobbyCreateTime"), (int64)10}};

	FJoinLobby::Params JoinLobbyParams;
	JoinLobbyParams.LocalName = TEXT("TestLobby");

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	LobbyCreateParams.LocalAccountId = AccountId;
	JoinLobbyParams.LocalAccountId = AccountId;
	
	LoginPipeline
		.EmplaceStep<FLobbyCreateHelper>(&LobbyCreateParams, [&JoinLobbyParams](UE::Online::FLobby InLobby) 
			{
				JoinLobbyParams.LobbyId = InLobby.LobbyId; 
			}, true);

	RunToCompletion();
}
