// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"
#include "Online/DelegateAdapter.h"
#include "Online/MulticastAdapter.h"

using namespace UE::Online;
class FAsyncTestStepHelper : public FAsyncTestStep
{
public:
	FAsyncTestStepHelper(bool& inBHasRun)
		: bHasRun(inBHasRun)
	{

	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		CHECK(Services != nullptr);
		bHasRun = true;
		Promise->SetValue(true);
	}

	bool& bHasRun;
};

#define AST_ERROR_TAG "[AsyncTestStep]"
#define AST_ERROR_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, AST_ERROR_TAG __VA_ARGS__)

AST_ERROR_TEST_CASE("Basic test- confirm all of async steps, lambdas, and async lambdas are compiling and executing properly")
{
	bool bHasRun = false;
	bool bDidComplete = false;

	GetPipeline()
		.EmplaceStep<FAsyncTestStepHelper>(bHasRun)
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			CHECK(bHasRun);
		})
		.EmplaceAsyncLambda([&](const FAsyncLambdaResult& Result, const IOnlineServicesPtr& Type)
		{
			CHECK(bHasRun);
			Result->SetValue(true);
		})
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			bDidComplete = true;
		});

	RunToCompletion();
	CHECK(bHasRun);
	CHECK(bDidComplete);
}