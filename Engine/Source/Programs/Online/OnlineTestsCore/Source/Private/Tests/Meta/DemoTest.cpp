// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"
 
#define DEMOTEST_TAG "[Demo]"  
#define DEMOTEST_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, DEMOTEST_TAG __VA_ARGS__)


DEMOTEST_TEST_CASE("Adding and subtracting numbers completes successfully")
{
	int32 a = 5;
	int32 b = 7;
	int32 c = 12;

	CHECK(a + b == 12);
}


DEMOTEST_TEST_CASE("Verify OSS selector working", "[.NULL]")
{
	CHECK(GetService() == TEXT("Null"));
}
