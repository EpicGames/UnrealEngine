// Copyright Epic Games, Inc. All Rights Reserved.

#include "AsyncTestStep.h"
#include "Online/DelegateAdapter.h"
#include "Online/MulticastAdapter.h"
#include "OnlineCatchHelper.h"

#define META_SUITE_TAGS "[Meta][Null]"
#define META_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, META_SUITE_TAGS)

using namespace UE::Online;

META_TEST_CASE("Async test steps")
{
	SECTION("Verify promise is fulfilled on async op failure - This test should fail (but the test framework should continue to function)")
	{
		bool bDidComplete = false;

		GetPipeline()
			.EmplaceAsyncLambda([&](FAsyncLambdaResult Result, const IOnlineServicesPtr& Type)
				{
					// This should cause a failure and crash, but if the fix is working
					// correctly, it should not prevent the promise from being fulfilled.
					REQUIRE(false);
				})
			.EmplaceLambda([&](const IOnlineServicesPtr& Type)
				{
					// Only runs if the previous step completed (the promise was fulfilled)
					bDidComplete = true;
				});

		RunToCompletion();
		CHECK(bDidComplete);
	}
}