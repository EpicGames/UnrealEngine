// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"

#define SELFTEST_TAG "[selftest][onlinecatchhelper]"
#define GENERATETAGS_TAG "[generatetags]"
#define CHECKALLTAGSISIN_TAG "[checkalltagsisin]"
#define SHOULDDISABLETEST_TAG "[shoulddisabletest]"
#define SELFTEST_TEST_CASE(x, ...) TEST_CASE(x, SELFTEST_TAG __VA_ARGS__)

#define RESETSERVICEMODULE_TAG "[resetservicemodules]"
#define SELFTEST_ONLINE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SELFTEST_TAG __VA_ARGS__)

SELFTEST_TEST_CASE("GenerateTags append MayFailTags case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append multiple match MayFailTags case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[bar]", "[foo]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match MayFailTags case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[wiz]", "[foo]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo],bar" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag no match case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo],[wiz]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags don't append MayFailTags no match case", GENERATETAGS_TAG)
{
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[wiz]" };
	FString TestTags = "[foo][bar]";
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags append ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[foo]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append multiple match ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[bar]", "[foo]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[wiz]", "[foo]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags don't append ShouldFail no match case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";	
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[wiz]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags appends MayFail and ShouldFail Case", GENERATETAGS_TAG)
{
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[foo]" };
	SkipTags.ShouldFailTags = { "[bar]" };
	FString TestTags = "[foo][bar]";
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!mayfail][!shouldfail]"));
}


SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[foo],bar" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar][!shouldfail]"));
}

SELFTEST_TEST_CASE("GenerateTags append by last match mutli-tag no match ShouldFail case", GENERATETAGS_TAG)
{
	FString TestTags = "[foo][bar]";
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.ShouldFailTags = { "[foo],[wiz]" };
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("GenerateTags appends no tags case", GENERATETAGS_TAG)
{
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.MayFailTags = { "[wiz]" };
	SkipTags.ShouldFailTags = {  };
	FString TestTags = "[foo][bar]";
	FString OutTags = OnlineAutoReg::GenerateTags("TestService", SkipTags, *TestTags);
	CHECK(OutTags == TEXT("[TestService] [foo][bar]"));
}

SELFTEST_TEST_CASE("ShouldDisableTest returns true on single-tag config", SHOULDDISABLETEST_TAG)
{
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.DisableTestTags = { "[foo]" };
	FString TestTags = "[foo][bar]";
	CHECK(OnlineAutoReg::ShouldDisableTest("TestService", SkipTags, *TestTags) == true);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns true on multi-tag config", SHOULDDISABLETEST_TAG)
{
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.DisableTestTags = { "[foo],bar" };
	FString TestTags = "[foo][bar]";
	CHECK(OnlineAutoReg::ShouldDisableTest("TestService", SkipTags, *TestTags) == true);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns true on !<service>", SHOULDDISABLETEST_TAG)
{
	FString TestTags = "[foo][bar][!TestService]";
	CHECK(OnlineAutoReg::ShouldDisableTest("TestService", {}, *TestTags) == true);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns false with no tags and no config skips", SHOULDDISABLETEST_TAG)
{
	FString TestTags = "[foo][bar]";
	CHECK(OnlineAutoReg::ShouldDisableTest("TestService", {}, *TestTags) == false);
}

SELFTEST_TEST_CASE("ShouldDisableTest returns false with no matching no-tags and no matching multi-tag config skips", SHOULDDISABLETEST_TAG)
{
	OnlineAutoReg::FReportingSkippableTags SkipTags;
	SkipTags.DisableTestTags = { "[foo],wiz" };
	FString TestTags = "[foo][bar]";
	CHECK(OnlineAutoReg::ShouldDisableTest("TestService", SkipTags, *TestTags) == false);
}

SELFTEST_TEST_CASE("CheckAllTagsIsIn(TArray, FString) true cases", CHECKALLTAGSISIN_TAG)
{
	TArray<FString> TestTags = { "bob", "alice", "foo" };
	CAPTURE(TestTags);

	// Truthy Cases
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "bob, alice") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "bob,alice") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, " bob,alice ") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "foo") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, ",foo") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "bob,alice,foo") == true);

	//Bracket Parsing
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "[bob],[alice],[foo]") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "[bob], [alice,foo]") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "bob],  alice],  [foo]  ,") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, ",[foo]") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, ",foo]") == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "[wiz]") == false);

	// Negative Cases
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "bob,alice,foo,wiz") == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "bob,wiz") == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, ",wiz") == false);

	// Bound Checks
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, ",") == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, "") == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn({}, "") == false);
}


SELFTEST_TEST_CASE("CheckAllTagsIsIn(TArray, TArray) true cases", CHECKALLTAGSISIN_TAG)
{
	TArray<FString> TestTags = { "bob", "alice", "foo" };
	CAPTURE(TestTags);

	// Truthy Cases
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, { "bob", "alice" }) == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, { "bob", "alice", "foo"}) == true);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, TArray<FString>({ "foo" })) == true);

	// Negative Cases
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, TArray<FString>({ "wiz" })) == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, { "bob", "alice", "foo", "wiz"}) == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, { "bob", "alice", "wiz" }) == false);

	// Bounds
	CHECK(OnlineAutoReg::CheckAllTagsIsIn(TestTags, TArray<FString>({})) == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn({}, TArray<FString>({})) == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn({}, TArray<FString>({"wiz"})) == false);
	CHECK(OnlineAutoReg::CheckAllTagsIsIn({"wiz"}, TArray<FString>({})) == false);
}
