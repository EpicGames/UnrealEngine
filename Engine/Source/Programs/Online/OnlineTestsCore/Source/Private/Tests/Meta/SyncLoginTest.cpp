// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"

#define SYNCLOGIN_ERROR_TAG "[NULL][Meta][SyncLogin]"
#define SYNCLOGIN_ERROR_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SYNCLOGIN_ERROR_TAG __VA_ARGS__)

SYNCLOGIN_ERROR_TEST_CASE("Confirm sync login is logging in before RunToCompletion is called")
{
	FAccountId AccountId, AccountId2;

	auto& PipeSave = GetLoginPipeline({ AccountId, AccountId2 });
	// Check is valid right after Pipeline call.
	CHECK(AccountId.IsValid());
	CHECK(AccountId2.IsValid());
	
	bool bHasRun = false;
	
	PipeSave
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
			{
				// Check valid during pipeline
				CHECK(AccountId.IsValid());
				CHECK(AccountId2.IsValid());
			})
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			bHasRun = true;
		});

	RunToCompletion();
	CHECK(bHasRun);
}

SYNCLOGIN_ERROR_TEST_CASE("Confirm sync login is logging in before RunToCompletion is called for five accounts")
{
	FAccountId AccountId, AccountId2, AccountId3, AccountId4, AccountId5;
	bool bHasRun = false;

	GetLoginPipeline({ AccountId, AccountId2, AccountId3, AccountId4, AccountId5 })
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
				// Check valid during pipeline
				CHECK(AccountId.IsValid());
				CHECK(AccountId2.IsValid());
				CHECK(AccountId3.IsValid());
				CHECK(AccountId4.IsValid());
				CHECK(AccountId5.IsValid());
		})
		.EmplaceLambda([&bHasRun](const IOnlineServicesPtr& Type)
		{
			bHasRun = true;
		});

	RunToCompletion();
	CHECK(bHasRun);
}