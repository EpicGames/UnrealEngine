// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Presence/BatchQueryPresenceHelper.h"
#include "Helpers/Presence/GetCachedPresenceHelper.h"
#include "Helpers/Presence/PartialUpdatePresenceHelper.h"
#include "Helpers/Presence/QueryPresenceHelper.h"
#include "Helpers/Presence/UpdatePresenceHelper.h"
#include "Helpers/Presence/CommonPresenceHelpers.h"

#define PRESENCE_TAGS "[Presence]"
#define PRESENCE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, PRESENCE_TAGS __VA_ARGS__)

PRESENCE_TEST_CASE("Ensure UpdatePresence works as expected and can then be queried with QueryPresence on self user")
{
	FAccountId AccountIdA;
	TSharedPtr<const FUserPresence> OutPresence;

	GetLoginPipeline({ AccountIdA })
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdA, GetGenericPresenceA(AccountIdA))
		.EmplaceStep<FQueryPresenceHelper>(AccountIdA, AccountIdA, OutPresence)
		.EmplaceStep<FComparePresencesHelper>(OutPresence, GetGenericPresenceA(AccountIdA));

	RunToCompletion();
}


PRESENCE_TEST_CASE("Ensure UpdatePresence works as expected and can then be queried with QueryPresence on non self user", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TSharedPtr<const FUserPresence> OutPresence;

	GetLoginPipeline({ AccountIdA, AccountIdB })
		// <todo: ensure friendship helper>
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdB, GetGenericPresenceA(AccountIdB))
		.EmplaceStep<FQueryPresenceHelper>(AccountIdA, AccountIdB, OutPresence)
		.EmplaceStep<FComparePresencesHelper>(OutPresence, GetGenericPresenceA(AccountIdB));

	RunToCompletion();
}

PRESENCE_TEST_CASE("Ensure UpdatePresence works on multiple users and can then be queried on all with QueryPresence (self and non-self)", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TArray<FAccountId> AccountIds;
	TArray<TSharedRef<const FUserPresence>> OutPresences;

	GetLoginPipeline({ AccountIdA, AccountIdB })
		.EmplaceLambda([&AccountIdA, &AccountIdB, &AccountIds](const IOnlineServicesPtr& Type)
		{
			AccountIds.Add(AccountIdA);
			AccountIds.Add(AccountIdB);
		})
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdA, GetGenericPresenceA(AccountIdA))
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdB, GetGenericPresenceB(AccountIdB))
		.EmplaceStep<FBatchQueryPresenceHelper>(AccountIdA, AccountIds, OutPresences)
		.EmplaceLambda([&AccountIdA, &AccountIdB, &OutPresences](const IOnlineServicesPtr& Type)
		{
			REQUIRE(OutPresences.Num() == 2);
			CheckPresenceAreEqual(*OutPresences[0], *GetGenericPresenceA(AccountIdA));
			CheckPresenceAreEqual(*OutPresences[1], *GetGenericPresenceB(AccountIdB));
		});

	RunToCompletion();
}

PRESENCE_TEST_CASE("Ensure PartialUpdatePresence works as expected and can then be queried with QueryPresence on non self user", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TSharedPtr<const FUserPresence> OutPresence;

	GetLoginPipeline({ AccountIdA, AccountIdB })
		// <todo: ensure friendship helper>
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdA, GetGenericPresenceA(AccountIdA))
		.EmplaceStep<FPartialUpdatePresenceHelper>(GetGenericPresenceMutationAtoB(AccountIdB))
		.EmplaceStep<FQueryPresenceHelper>(AccountIdA, AccountIdB, OutPresence)
		.EmplaceStep<FComparePresencesHelper>(OutPresence, GetGenericPresenceB(AccountIdB));

	RunToCompletion();
}

PRESENCE_TEST_CASE("Ensure UpdatePresence works as expected and can then be queried with GetCachedPresence on self user")
{
	FAccountId AccountIdA;
	TSharedPtr<const FUserPresence> OutPresenceA, OutPresenceB;

	GetLoginPipeline({ AccountIdA })
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdA, GetGenericPresenceA(AccountIdA))
		.EmplaceStep<FQueryPresenceHelper>(AccountIdA, AccountIdA, OutPresenceA)
		.EmplaceStep<FGetCachedPresenceHelper>(AccountIdA, AccountIdA, OutPresenceB)
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			CheckPresenceAreEqual(*OutPresenceA, *OutPresenceB);
		});

	RunToCompletion();
}

PRESENCE_TEST_CASE("Ensure QueryPresence's bListenToChanges mod fires the event properly", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TSharedPtr<const FUserPresence> OutPresence;
	bool bReceivedPresence = false;

	auto Handle = GetServices()->GetPresenceInterface()->OnPresenceUpdated().Add([&](const FPresenceUpdated& Event) mutable
	{
		if (Event.UpdatedPresence->AccountId == AccountIdB)
		{
			bReceivedPresence = true;
			CheckPresencesNotEqual(*OutPresence, *Event.UpdatedPresence);
		}
	});

	GetLoginPipeline({ AccountIdA, AccountIdB })
		// <todo: ensure friendship helper>
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdB, GetGenericPresenceA(AccountIdB))
		.EmplaceStep<FQueryPresenceHelper>(AccountIdA, AccountIdB, OutPresence, true)
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdB, GetGenericPresenceB(AccountIdB))
		// todo: possibly add a fixed delay here or a waited retry on the event
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			CHECK(bReceivedPresence);
		});

	RunToCompletion();
	// todo: events don't seem to have a remover for the handle?
}


PRESENCE_TEST_CASE("Ensure BatchQueryPresence's bListenToChanges mod fires the event properly", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TArray<FAccountId> AccountIds;
	TArray<TSharedRef<const FUserPresence>> OutPresences;

	bool bReceivedPresence = false;

	auto Handle = GetServices()->GetPresenceInterface()->OnPresenceUpdated().Add([&](const FPresenceUpdated& Event)
	{
		if (Event.UpdatedPresence->AccountId == AccountIdB)
		{
			bReceivedPresence = true;
		}
	});

	GetLoginPipeline({ AccountIdA, AccountIdB })
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			AccountIds.Add(AccountIdA);
			AccountIds.Add(AccountIdB);
		})
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdA, GetGenericPresenceA(AccountIdA))
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdB, GetGenericPresenceA(AccountIdB))
		.EmplaceStep<FBatchQueryPresenceHelper>(AccountIdA, AccountIds, OutPresences, true)
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdA, GetGenericPresenceB(AccountIdA))
		.EmplaceStep<FUpdatePresenceHelper>(AccountIdB, GetGenericPresenceB(AccountIdB))
		// todo: possibly add a fixed delay here or a waited retry on the event
		.EmplaceLambda([&](const IOnlineServicesPtr& Type)
		{
			CHECK(bReceivedPresence);
		});


	RunToCompletion();
}

// CreatePresenceStruct tests
PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct and stores it in the given variable shortname")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct UserId")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct Status")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct Joinability")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct GameStatus")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct StatusString")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct RichPresenceString")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that CreatePresenceStruct creates new presence struct with correct Properties")
{
	// TODO
}

// QueryPresence tests
//PRESENCE_TEST_CASE("Verify QueryPresence caches the presence of an existing user and displays a Success message") // Already covered
//{
//	// TODO
//}

PRESENCE_TEST_CASE("Verify QueryPresence displays a Fail message when given a nonexistent user")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify QueryPresence caches the presence of the local user when given the UserId 0")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify QueryPresence does not change an already cached presence if it has not been updated")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify QueryPresence updates an already cached presence if it has been updated")
{
	// TODO
}

// BatchQueryPresence tests
PRESENCE_TEST_CASE("Verify BatchQueryPresence caches the presence of one existing user and displays a Success message")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify BatchQueryPresence caches the presence of two existing users and displays a Success message")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify BatchQueryPresence displays a Fail message when given one nonexistent user")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify BatchQueryPresence displays a Fail message when given two nonexistent users")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify BatchQueryPresence displays a Fail message when given one existing user and one nonexistent user and does not cache the presence of the existing one")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify BatchQueryPresence does not change an already cached presence that has not been updated")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify BatchQueryPresence updates an already cached presence if it has been updated")
{
	// TODO
}

// GetCachedPresence tests
//PRESENCE_TEST_CASE("Verify GetCachedPresence outputs the presence of the given user to the log if it is cached") // Already covered
//{
//	// TODO
//}

PRESENCE_TEST_CASE("Verify GetCachedPresence outputs the presence of the given user to the log when there is another presence cached")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify GetCachedPresence returns an error when the given user has no presence cached")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify GetCachedPresence outputs the updated presence after it has been updated and recached")
{
	// TODO
}

// UpdatePresence tests
//PRESENCE_TEST_CASE("Verify UpdatePresence updates the given user with the specified presence detailed in parameters") // Already covered
//{
//	// TODO
//}

PRESENCE_TEST_CASE("Verify UpdatePresence updates the given user with the specified presence by shortname")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify UpdatePresence displays a Fail message when the given user does not exist")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify UpdatePresence displays a Fail message when given a nonexistent presence shortname")
{
	// TODO
}

// PartialUpdatePresence
//PRESENCE_TEST_CASE("Verify PartialUpdatePresence updates the given user with the specified presence detailed in parameters") // Already covered
//{
//	// TODO
//}

PRESENCE_TEST_CASE("Verify PartialUpdatePresence updates the given user with the specified presence by shortname")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify PartialUpdatePresence displays a Fail message when the given user does not exist")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify PartialUpdatePresence displays a Fail message when given a nonexistent presence shortname")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify PartialUpdatePresence does not update presence parameters that are given as null")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify PartialUpdatePresence removes keys listed for removal")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify PartialUpdatePresence displays a Fail message when a nonexistent key is listed for removal")
{
	// TODO
}

// AddEventDebugListener
PRESENCE_TEST_CASE("Verify that AddEventDebugListener causes the local user's log to display when a Friend calls UpdatePresence")
{
	// TODO
}

PRESENCE_TEST_CASE("Verify that after AddEventDebugListener is called, if a Friend unfriends the local user, their UpdatePresence calls do not appear in the local user's log")
{
	// TODO
}