// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnlineCatchHelper.h"
#include "Online/Schema.h"

#define SCHEMA_SUITE_TAGS "[Schema]"
#define HIDE_DEFAULT_SUITE_TAG "[.]"
#define SCHEMA_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, HIDE_DEFAULT_SUITE_TAG SCHEMA_SUITE_TAGS __VA_ARGS__)

using namespace UE::Online;

SCHEMA_TEST_CASE("Config parsing - Schema id validity")
{
	FSchemaId TestSchemaId1 = "Schema1";
	FSchemaId TestSchemaId2 = "Schema2";
	FSchemaCategoryId TestCategoryId1 = "Category1";

	SECTION("Multiple schema")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Duplicated schema id")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Config parsing - Schema parent relationship")
{
	FSchemaId TestSchemaId1 = "Schema1";
	FSchemaId TestSchemaId2 = "Schema2";
	FSchemaId TestSchemaId3 = "Schema3";
	FSchemaCategoryId TestCategoryId1 = "Category1";

	SECTION("Parented schema")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId3, TestSchemaId2 });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Parented schema with circular dependency")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, TestSchemaId3 });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId3, TestSchemaId2 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Missing parent")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Config parsing - Schema categories")
{
	FSchemaId TestSchemaId1 = "Schema1";
	FSchemaId TestSchemaId2 = "Schema2";
	FSchemaCategoryId TestCategoryId1 = "Category1";
	FSchemaCategoryId TestCategoryId2 = "Category2";

	SECTION("Category specified by parent schema descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId()});

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Category specified by child schema descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId() });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1, { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Category not specified in base schema descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate category id in schema descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1, TestCategoryId1 }});
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate category descriptors")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Config parsing - Schema service descriptor")
{
	FSchemaId TestSchemaId1 = "Schema1";
	FSchemaId TestSchemaId2 = "Schema2";
	FSchemaCategoryId TestCategoryId1 = "Category1";
	FSchemaCategoryId TestCategoryId2 = "Category2";
	FSchemaServiceDescriptorId TestServiceDescriptorId1 = "ServiceDescriptor1";
	FSchemaServiceDescriptorId TestServiceDescriptorId2 = "ServiceDescriptor2";

	SECTION("Service descriptor exists")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, TestServiceDescriptorId1 });
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Service descriptor does not exist")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, TestServiceDescriptorId1 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Service descriptor not reused between categories")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1, TestCategoryId2 }});
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, TestServiceDescriptorId1 });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId2, TestServiceDescriptorId1 });
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate service descriptor id")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, TestServiceDescriptorId1 });
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1 });
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Config parsing - Schema attributes")
{
	FSchemaId TestSchemaId1 = "Schema1";
	FSchemaId TestSchemaId2 = "Schema2";
	FSchemaCategoryId TestCategoryId1 = "Category1";
	FSchemaAttributeId TestAttributeId1 = "Attribute1";
	FSchemaAttributeId TestAttributeId2 = "Attribute2";

	SECTION("Valid attribute")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Invalid attribute type")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::None, { ESchemaAttributeFlags::Public }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Invalid attribute visibility - none")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, {}, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Invalid attribute visibility")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::Private }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Invalid attribute searchability")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Searchable, ESchemaAttributeFlags::Private }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate attribute")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });

		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Valid max length")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Invalid max length 0")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Invalid max length -1")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, -1 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Searchable attribute declared in base")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::Searchable }, 0, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Searchable attribute declared in derived")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId2, TestCategoryId1, { TestAttributeId1 }});

		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::Searchable }, 0, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("SchemaCompatibilityId declared in base")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("SchemaCompatibilityId declared in derived")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });

		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId2, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Multiple SchemaCompatibilityId declared")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1, TestAttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("SchemaCompatibilityId incorrect type")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("SchemaCompatibilityId incorrect visibility")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Private, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Valid UpdateGroupId")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 1, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Invalid UpdateGroupId")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, -1, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Config parsing - Schema category attributes")
{
	FSchemaId TestSchemaId1 = "Schema1";
	FSchemaId TestSchemaId2 = "Schema2";
	FSchemaCategoryId TestCategoryId1 = "Category1";
	FSchemaCategoryId TestCategoryId2 = "Category2";
	FSchemaAttributeId TestAttributeId1 = "Attribute1";
	FSchemaAttributeId TestAttributeId2 = "Attribute2";

	SECTION("Valid attribute added by base class")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Valid attribute added by child class")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId2, TestCategoryId1, { TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Duplicate category attribute descriptors")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate attribute id in category attribute descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1, TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Attribute added to missing category in base")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId2, { TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Attribute added to missing category in child")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId2, TestCategoryId2, { TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Attribute duplicated in child node")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId1, FSchemaId(), { TestCategoryId1 } });
		SchemaConfig.SchemaDescriptors.Add({ TestSchemaId2, TestSchemaId1 });
		SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId1, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId1, TestCategoryId1, { TestAttributeId1 } });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId2, TestCategoryId1, { TestAttributeId1 } });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Config parsing - Service descriptor")
{
	FSchemaServiceDescriptorId TestServiceDescriptorId1 = "ServiceDescriptor1";
	FSchemaServiceAttributeId TestAttributeId1 = "Attribute1";

	SECTION("Valid service attribute id")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1, { TestAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::String }, { ESchemaServiceAttributeFlags::Public, ESchemaServiceAttributeFlags::Private, ESchemaServiceAttributeFlags::Searchable }, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Invalid service attribute type")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1, { TestAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::None }, { ESchemaServiceAttributeFlags::Public }, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Invalid service attribute visibility")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1, { TestAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::None }, {}, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Inalid service attribute max length")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1, { TestAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::String }, { ESchemaServiceAttributeFlags::Public }, 0 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate attribute id in service descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1, { TestAttributeId1, TestAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::String }, {}, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}

	SECTION("Duplicate service attribute descriptor")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.ServiceDescriptors.Add({ TestServiceDescriptorId1, { TestAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::String }, { ESchemaServiceAttributeFlags::Public }, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ TestAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::String }, { ESchemaServiceAttributeFlags::Public }, 32 });

		FSchemaRegistry SchemaRegistry;
		CHECK(SchemaRegistry.ParseConfig(SchemaConfig) == false);
	}
}

SCHEMA_TEST_CASE("Schema definition - no backing service")
{
	FSchemaId TestSchemaId = "Schema";
	FSchemaAttributeId TestAttributeId = "Attribute1";
	FSchemaCategoryId TestCategoryId = "Default";

	FSchemaRegistryDescriptorConfig SchemaConfig;
	SchemaConfig.SchemaDescriptors.Add({ TestSchemaId, FSchemaId(), { TestCategoryId } });
	SchemaConfig.SchemaCategoryDescriptors.Add({ TestCategoryId, FSchemaServiceDescriptorId() });
	SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ TestSchemaId, TestCategoryId, { TestAttributeId } });
	SchemaConfig.SchemaAttributeDescriptors.Add({ TestAttributeId, ESchemaAttributeType::Bool, { ESchemaAttributeFlags::Public }, 0, 0 });

	FSchemaRegistry SchemaRegistry;
	REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

	TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(TestSchemaId);
	REQUIRE(SchemaDefinition.IsValid());
	CHECK(SchemaDefinition->CompatibilityId != 0);

	const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(TestCategoryId);
	REQUIRE(CategoryDefinition != nullptr);
	CHECK(CategoryDefinition->SchemaAttributeDefinitions.Num() == 1);
	CHECK(CategoryDefinition->ServiceAttributeDefinitions.IsEmpty());

	const FSchemaAttributeDefinition* AttributeDefinition = CategoryDefinition->SchemaAttributeDefinitions.Find(TestAttributeId);
	REQUIRE(AttributeDefinition != nullptr);
	CHECK(AttributeDefinition->Id == TestAttributeId);
	CHECK(AttributeDefinition->Type == ESchemaAttributeType::Bool);
	CHECK(AttributeDefinition->Flags == ESchemaAttributeFlags::Public);
	CHECK(AttributeDefinition->MaxSize == 0);
	CHECK(AttributeDefinition->ServiceAttributeId == FSchemaServiceAttributeId());
}

SCHEMA_TEST_CASE("Schema definition")
{
	const FSchemaId BaseSchemaId = "BaseSchema";
	const FSchemaId SchemaId = "Schema";
	const FSchemaCategoryId CategoryId1 = "Category1";
	const FSchemaAttributeId AttributeId1 = "Attribute1";
	const FSchemaAttributeId AttributeId2 = "Attribute2";
	const FSchemaAttributeId AttributeId3 = "Attribute3";
	const FSchemaServiceDescriptorId ServiceDescriptorId = "ServiceDescriptor";
	const FSchemaServiceAttributeId ServiceAttributeId1 = "ServiceAttribute1";
	const FSchemaServiceAttributeId ServiceAttributeId2 = "ServiceAttribute2";
	const FSchemaServiceAttributeId ServiceAttributeId3 = "ServiceAttribute3";
	const TArray<ESchemaServiceAttributeSupportedTypeFlags> AllSupportedServiceAttributeTypes = {
		ESchemaServiceAttributeSupportedTypeFlags::Bool,
		ESchemaServiceAttributeSupportedTypeFlags::Int64,
		ESchemaServiceAttributeSupportedTypeFlags::Double,
		ESchemaServiceAttributeSupportedTypeFlags::String };
	const TArray<ESchemaServiceAttributeFlags> AllSupportedServiceAttributeFlags = {
		ESchemaServiceAttributeFlags::Searchable,
		ESchemaServiceAttributeFlags::Public,
		ESchemaServiceAttributeFlags::Private };

	SECTION("Valid assignment - No backing service")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(SchemaId);
		REQUIRE(SchemaDefinition.IsValid());
		CHECK(SchemaDefinition->CompatibilityId != 0);

		TSharedPtr<const FSchemaDefinition> SchemaDefinitionByCompatibilityId = SchemaRegistry.GetDefinition(SchemaDefinition->CompatibilityId);
		REQUIRE(SchemaDefinitionByCompatibilityId.IsValid());
		CHECK(SchemaDefinition->Id == SchemaDefinitionByCompatibilityId->Id);

		const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(CategoryId1);
		REQUIRE(CategoryDefinition != nullptr);
		CHECK(CategoryDefinition->SchemaAttributeDefinitions.Num() == 1);
		CHECK(CategoryDefinition->ServiceAttributeDefinitions.Num() == 0);

		const FSchemaAttributeDefinition* AttributeDefinition = CategoryDefinition->SchemaAttributeDefinitions.Find(AttributeId1);
		REQUIRE(AttributeDefinition != nullptr);
		CHECK(AttributeDefinition->Id == AttributeId1);
		CHECK(AttributeDefinition->Type == ESchemaAttributeType::String);
		CHECK(AttributeDefinition->Flags == ESchemaAttributeFlags::Public);
		CHECK(AttributeDefinition->MaxSize == 32);
		CHECK(AttributeDefinition->ServiceAttributeId == FSchemaServiceDescriptorId());
	}

	SECTION("Valid assignment - Backing service")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(SchemaId);
		REQUIRE(SchemaDefinition.IsValid());
		CHECK(SchemaDefinition->CompatibilityId != 0);

		TSharedPtr<const FSchemaDefinition> SchemaDefinitionByCompatibilityId = SchemaRegistry.GetDefinition(SchemaDefinition->CompatibilityId);
		REQUIRE(SchemaDefinitionByCompatibilityId.IsValid());
		CHECK(SchemaDefinition->Id == SchemaDefinitionByCompatibilityId->Id);

		const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(CategoryId1);
		REQUIRE(CategoryDefinition != nullptr);
		CHECK(CategoryDefinition->SchemaAttributeDefinitions.Num() == 1);
		CHECK(CategoryDefinition->ServiceAttributeDefinitions.Num() == 1);

		const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = CategoryDefinition->ServiceAttributeDefinitions.Find(ServiceAttributeId1);
		REQUIRE(ServiceAttributeDefinition != nullptr);
		CHECK(ServiceAttributeDefinition->Id == ServiceAttributeId1);
		CHECK(ServiceAttributeDefinition->Type == ESchemaServiceAttributeSupportedTypeFlags::String);
		CHECK(ServiceAttributeDefinition->Flags == ESchemaServiceAttributeFlags::Public);
		CHECK(ServiceAttributeDefinition->MaxSize == 32);
		CHECK(ServiceAttributeDefinition->SchemaAttributeIds.Find(AttributeId1) != INDEX_NONE);

		const FSchemaAttributeDefinition* AttributeDefinition = CategoryDefinition->SchemaAttributeDefinitions.Find(AttributeId1);
		REQUIRE(AttributeDefinition != nullptr);
		CHECK(AttributeDefinition->Id == AttributeId1);
		CHECK(AttributeDefinition->Type == ESchemaAttributeType::String);
		CHECK(AttributeDefinition->Flags == ESchemaAttributeFlags::Public);
		CHECK(AttributeDefinition->MaxSize == 32);
		CHECK(AttributeDefinition->ServiceAttributeId == ServiceAttributeDefinition->Id);
	}

	SECTION("Not enough public service attributes")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, { ESchemaServiceAttributeFlags::Private }, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(!SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Not enough private service attributes")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, { ESchemaServiceAttributeFlags::Public }, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(!SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Not enough search service attributes")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::Searchable }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, { ESchemaServiceAttributeFlags::Private }, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(!SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Valid search service attributes")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::Searchable }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, { ESchemaServiceAttributeFlags::Public, ESchemaServiceAttributeFlags::Searchable }, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(SchemaId);
		REQUIRE(SchemaDefinition.IsValid());

		const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(CategoryId1);
		REQUIRE(CategoryDefinition != nullptr);
		CHECK(CategoryDefinition->ServiceAttributeDefinitions.Num() == 1);

		const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = CategoryDefinition->ServiceAttributeDefinitions.Find(ServiceAttributeId1);
		REQUIRE(ServiceAttributeDefinition != nullptr);
		CHECK(ServiceAttributeDefinition->Type == ESchemaServiceAttributeSupportedTypeFlags::String);
		CHECK(ServiceAttributeDefinition->Flags == ESchemaServiceAttributeFlags(ESchemaServiceAttributeFlags::Public | ESchemaServiceAttributeFlags::Searchable));
	}

	SECTION("Not enough service attributes of string capability")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::Bool }, AllSupportedServiceAttributeFlags, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(!SchemaRegistry.ParseConfig(SchemaConfig));
	}

	SECTION("Not enough service attributes of string size")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, { ESchemaServiceAttributeSupportedTypeFlags::String }, AllSupportedServiceAttributeFlags, 31 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(!SchemaRegistry.ParseConfig(SchemaConfig));
	}

	// Verify that base schema attributes are assigned before derived schema.
	SECTION("Base schema attribute ordering 1")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId, BaseSchemaId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId2, AttributeId3 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId3, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2, ServiceAttributeId3 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId3, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(BaseSchemaId);
		REQUIRE(SchemaDefinition.IsValid());

		const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(CategoryId1);
		REQUIRE(CategoryDefinition != nullptr);
		CHECK(CategoryDefinition->ServiceAttributeDefinitions.Num() == 1);

		const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = CategoryDefinition->ServiceAttributeDefinitions.Find(ServiceAttributeId1);
		REQUIRE(ServiceAttributeDefinition != nullptr);
		CHECK(ServiceAttributeDefinition->Type == ESchemaServiceAttributeSupportedTypeFlags::Int64);
		CHECK(ServiceAttributeDefinition->Flags == ESchemaServiceAttributeFlags(ESchemaServiceAttributeFlags::Public));
	}

	// Verify that base schema attributes are assigned before derived schema.
	SECTION("Base schema attribute ordering 2")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId, CategoryId1, { AttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId, BaseSchemaId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId3, AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId3, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2, ServiceAttributeId3 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId3, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(BaseSchemaId);
		REQUIRE(SchemaDefinition.IsValid());

		const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(CategoryId1);
		REQUIRE(CategoryDefinition != nullptr);
		CHECK(CategoryDefinition->ServiceAttributeDefinitions.Num() == 1);

		const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = CategoryDefinition->ServiceAttributeDefinitions.Find(ServiceAttributeId1);
		REQUIRE(ServiceAttributeDefinition != nullptr);
		CHECK(ServiceAttributeDefinition->Type == ESchemaServiceAttributeSupportedTypeFlags::Int64);
		CHECK(ServiceAttributeDefinition->Flags == ESchemaServiceAttributeFlags(ESchemaServiceAttributeFlags::Public));
	}

	// Verify that base schema attributes are assigned before derived schema.
	SECTION("Base schema attribute ordering 3")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId, CategoryId1, { AttributeId3 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId3, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId, BaseSchemaId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId, CategoryId1, { AttributeId1, AttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2, ServiceAttributeId3 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId3, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		FSchemaRegistry SchemaRegistry;
		REQUIRE(SchemaRegistry.ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry.GetDefinition(BaseSchemaId);
		REQUIRE(SchemaDefinition.IsValid());

		const FSchemaCategoryDefinition* CategoryDefinition = SchemaDefinition->Categories.Find(CategoryId1);
		REQUIRE(CategoryDefinition != nullptr);
		CHECK(CategoryDefinition->ServiceAttributeDefinitions.Num() == 1);

		const FSchemaServiceAttributeDefinition* ServiceAttributeDefinition = CategoryDefinition->ServiceAttributeDefinitions.Find(ServiceAttributeId1);
		REQUIRE(ServiceAttributeDefinition != nullptr);
		CHECK(ServiceAttributeDefinition->Type == ESchemaServiceAttributeSupportedTypeFlags::Int64);
		CHECK(ServiceAttributeDefinition->Flags == ESchemaServiceAttributeFlags(ESchemaServiceAttributeFlags::Public));
	}
}

SCHEMA_TEST_CASE("Schema category instance")
{
	const FSchemaId BaseSchemaId1 = "BaseSchema1";
	const FSchemaId BaseSchemaId2 = "BaseSchema2";
	const FSchemaId SchemaId1 = "Schema1";
	const FSchemaId SchemaId2 = "Schema2";
	const FSchemaCategoryId CategoryId1 = "Category1";
	const FSchemaCategoryId CategoryId2 = "Category2";
	const FSchemaAttributeId AttributeId1 = "Attribute1";
	const FSchemaAttributeId AttributeId2 = "Attribute2";
	const FSchemaAttributeId AttributeId3 = "Attribute3";
	const FSchemaServiceDescriptorId ServiceDescriptorId = "ServiceDescriptor";
	const FSchemaServiceAttributeId ServiceAttributeId1 = "ServiceAttribute1";
	const FSchemaServiceAttributeId ServiceAttributeId2 = "ServiceAttribute2";
	const FSchemaServiceAttributeId ServiceAttributeId3 = "ServiceAttribute3";
	const TArray<ESchemaServiceAttributeSupportedTypeFlags> AllSupportedServiceAttributeTypes = {
		ESchemaServiceAttributeSupportedTypeFlags::Bool,
		ESchemaServiceAttributeSupportedTypeFlags::Int64,
		ESchemaServiceAttributeSupportedTypeFlags::Double,
		ESchemaServiceAttributeSupportedTypeFlags::String };
	const TArray<ESchemaServiceAttributeFlags> AllSupportedServiceAttributeFlags = {
		ESchemaServiceAttributeFlags::Searchable,
		ESchemaServiceAttributeFlags::Public,
		ESchemaServiceAttributeFlags::Private };

	SECTION("Invalid category")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance(FSchemaId(), SchemaId1, CategoryId2, SchemaRegistry);
		CHECK(!CategoryInstance.IsValid());
	}

	SECTION("Invalid schema id")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance(FSchemaId(), SchemaId2, CategoryId1, SchemaRegistry);
		CHECK(!CategoryInstance.IsValid());
	}

	SECTION("Valid instance - no backing service - no parent")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		TSharedPtr<const FSchemaDefinition> SchemaDefinition = SchemaRegistry->GetDefinition(SchemaId1);
		REQUIRE(SchemaDefinition.IsValid());

		FSchemaCategoryInstance CategoryInstance(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance.IsValid());

		FSchemaServiceAttributeId SchemaServiceAttributeId;
		ESchemaServiceAttributeFlags SchemaServiceAttributeFlags;
		FSchemaVariant AttributeData;
		// wrong type
		AttributeData.Set(true);
		CHECK(!CategoryInstance.VerifyBaseAttributeData(AttributeId1, AttributeData, SchemaServiceAttributeId, SchemaServiceAttributeFlags));
		// valid string
		AttributeData.Set(TEXT("Valid string"));
		CHECK(CategoryInstance.VerifyBaseAttributeData(AttributeId1, AttributeData, SchemaServiceAttributeId, SchemaServiceAttributeFlags));
		// invalid string
		AttributeData.Set(TEXT("Too long string ========================================"));
		CHECK(!CategoryInstance.VerifyBaseAttributeData(AttributeId1, AttributeData, SchemaServiceAttributeId, SchemaServiceAttributeFlags));

		CHECK(CategoryInstance.GetDerivedDefinition() == TSharedPtr<const FSchemaDefinition>());
		CHECK(CategoryInstance.GetBaseDefinition() == SchemaDefinition);
	}

	SECTION("Valid instance - no backing service - no parent - serialize")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, FSchemaServiceDescriptorId() });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		CHECK(CategoryInstance.IsValid());

		// Serializing with no backing service will apply the attributes to the cache within CategoryInstance.
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId1, TEXT("Valid string") } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance.PrepareClientChanges(MoveTemp(ClientChangesParams));
		REQUIRE(PrepareClientChangesResult.IsOk());
	}

	SECTION("Valid instance - backing service - no parent")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		CHECK(CategoryInstance.IsValid());
	}

	SECTION("Valid instance - backing service - invalid parent")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry);
		CHECK(!CategoryInstance.IsValid());
	}

	SECTION("Valid instance - backing service - parent")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1, AttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId3 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId3, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2, ServiceAttributeId3 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId3, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance.IsValid());

		FSchemaServiceAttributeId SchemaServiceAttributeId;
		ESchemaServiceAttributeFlags SchemaServiceAttributeFlags;
		FSchemaVariant AttributeData;
		// valid string
		AttributeData.Set(TEXT("Valid string"));
		CHECK(CategoryInstance.VerifyBaseAttributeData(AttributeId2, AttributeData, SchemaServiceAttributeId, SchemaServiceAttributeFlags));
		// invalid base attribute
		AttributeData.Set(TEXT("Valid string"));
		CHECK(!CategoryInstance.VerifyBaseAttributeData(AttributeId3, AttributeData, SchemaServiceAttributeId, SchemaServiceAttributeFlags));
	}

	SECTION("Valid instance - backing service - valid serialization")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		FSchemaVariant AttributeData(TEXT("Valid string"));
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({{ AttributeId1, AttributeData }});
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		REQUIRE(PrepareClientChangesResult.IsOk());
		CHECK(PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Num() == 1);
		CHECK(PrepareClientChangesResult.GetOkValue().ServiceChanges.RemovedAttributes.IsEmpty());

		FSchemaServiceAttributeData* SchemaServiceAttributeData = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttributeData);

		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({{SchemaServiceAttributeData->Id, SchemaServiceAttributeData->Value}});
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
		REQUIRE(PrepareServiceSnapshotResult.IsOk());

		FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshotResult = CategoryInstance2.CommitServiceSnapshot();
		FSchemaVariant* SchemaAttributeData = CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Find(AttributeId1);
		REQUIRE(SchemaAttributeData);
		CHECK(*SchemaAttributeData == AttributeData);
	}

	SECTION("Valid instance - backing service - invalid serialization - too large")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		FSchemaVariant AttributeData(TEXT("Too long string ========================================"));
		FSchemaCategoryInstancePrepareClientChanges::Params Params;
		Params.ClientChanges.UpdatedAttributes = { { AttributeId1, AttributeData } };
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> Result = CategoryInstance1.PrepareClientChanges(MoveTemp(Params));
		CHECK(!Result.IsOk());
		CHECK(Result.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - invalid serialization - wrong type")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		FSchemaVariant AttributeData(true);
		FSchemaCategoryInstancePrepareClientChanges::Params Params;
		Params.ClientChanges.UpdatedAttributes = { { AttributeId1, AttributeData } };
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> Result = CategoryInstance1.PrepareClientChanges(MoveTemp(Params));
		CHECK(!Result.IsOk());
		CHECK(Result.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - invalid serialization - unknown attribute")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::String, { ESchemaAttributeFlags::Public }, 0, 32 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		FSchemaVariant AttributeData(TEXT("Valid string"));
		FSchemaCategoryInstancePrepareClientChanges::Params Params;
		Params.ClientChanges.UpdatedAttributes = { { AttributeId2, AttributeData } };
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> Result = CategoryInstance1.PrepareClientChanges(MoveTemp(Params));
		CHECK(!Result.IsOk());
		CHECK(Result.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Attempt to set SchemaCompatibilityId externally")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId1, FSchemaVariant(0ll) } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(!PrepareClientChangesResult.IsOk());
		CHECK(PrepareClientChangesResult.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Attempt to remove SchemaCompatibilityId externally")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();;
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(FSchemaId(), SchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.RemovedAttributes = { AttributeId1 };
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(!PrepareClientChangesResult.IsOk());
		CHECK(PrepareClientChangesResult.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Read with different SchemaCompatibilityId - different base")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Private }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId2, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId2, CategoryId1, { AttributeId1 } });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId2, BaseSchemaId2 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(SchemaId2, BaseSchemaId2, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, FSchemaVariant(42ll) } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(PrepareClientChangesResult.IsOk());

		FSchemaServiceAttributeData* SchemaServiceAttributeData = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttributeData);

		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({{ SchemaServiceAttributeData->Id, SchemaServiceAttributeData->Value }});
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
		REQUIRE(!PrepareServiceSnapshotResult.IsOk());
		CHECK(PrepareServiceSnapshotResult.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Read with different SchemaCompatibilityId - valid")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Private }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId2, BaseSchemaId1 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(SchemaId2, BaseSchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		// Write value to instance 1.
		FSchemaVariant VariantData(42ll);
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, VariantData } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(PrepareClientChangesResult.IsOk());

		// Grab service output.
		FSchemaServiceAttributeData* SchemaServiceAttribute1Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttribute1Data);
		FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
		REQUIRE(SchemaServiceAttribute2Data);

		// Feed service output into instance 2.
		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
			{ SchemaServiceAttribute1Data->Id, SchemaServiceAttribute1Data->Value },
			{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value }});
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
		REQUIRE(PrepareServiceSnapshotResult.IsOk());

		// Check that the schema in instance 2 changed from SchemaId2 to SchemaId1.
		FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshotResult = CategoryInstance2.CommitServiceSnapshot();
		REQUIRE(CommitServiceSnapshotResult.ClientChanges.SchemaId.IsSet());
		CHECK(*CommitServiceSnapshotResult.ClientChanges.SchemaId == SchemaId1);
		REQUIRE(CategoryInstance2.GetDerivedDefinition() != nullptr);
		CHECK(CategoryInstance2.GetDerivedDefinition()->Id == SchemaId1);

		// Check that attribute data defined in SchemaId1 is received in instance 2.
		CHECK(CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Num() == 1);
		FSchemaVariant* SchemaAttributeData = CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Find(AttributeId2);
		REQUIRE(SchemaAttributeData != nullptr);
		CHECK(*SchemaAttributeData == VariantData);
	}

	SECTION("Valid instance - backing service - Read with different SchemaCompatibilityId - unknown schema")
	{
		TSharedRef<FSchemaRegistry> SchemaRegistry1 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;
			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Private }, 0, 0 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry1->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry1);
		REQUIRE(CategoryInstance1.IsValid());

		TSharedRef<FSchemaRegistry> SchemaRegistry2 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId2, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId2, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Private }, 0, 0 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry2->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance2(SchemaId2, BaseSchemaId1, CategoryId1, SchemaRegistry2);
		REQUIRE(CategoryInstance2.IsValid());

		// Write value to instance 1.
		FSchemaVariant VariantData(42ll);
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, VariantData } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(PrepareClientChangesResult.IsOk());

		// Grab service output.
		FSchemaServiceAttributeData* SchemaServiceAttribute1Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttribute1Data);
		FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
		REQUIRE(SchemaServiceAttribute2Data);

		// Feed service output into instance 2.
		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
			{ SchemaServiceAttribute1Data->Id, SchemaServiceAttribute1Data->Value },
			{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value }});
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));

		// Check that the schema in instance 2 changed from SchemaId2 to SchemaId1.
		REQUIRE(!PrepareServiceSnapshotResult.IsOk());
		CHECK(PrepareServiceSnapshotResult.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Read with same SchemaCompatibilityId - attribute changes")
	{
		TSharedRef<FSchemaRegistry> SchemaRegistry1 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 32 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry1->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry1);
		REQUIRE(CategoryInstance1.IsValid());

		TSharedRef<FSchemaRegistry> SchemaRegistry2 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 32 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			REQUIRE(SchemaRegistry2->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance2(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry2);
		REQUIRE(CategoryInstance2.IsValid());

		FSchemaVariant StartingVariantData(TEXT("bacon"));
		FSchemaVariant ChangedVariantData(TEXT("waffles"));

		// Check attribute was added.
		{
			// Write value to instance 1.
			FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
			ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, StartingVariantData } });
			TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
			CHECK(PrepareClientChangesResult.IsOk());

			// Grab service output.
			FSchemaServiceAttributeData* SchemaServiceAttribute1Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
			REQUIRE(SchemaServiceAttribute1Data);
			FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
			REQUIRE(SchemaServiceAttribute2Data);

			// Feed service output into instance 2.
			FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
			PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
				{ SchemaServiceAttribute1Data->Id, SchemaServiceAttribute1Data->Value },
				{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value }});
			TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
			REQUIRE(PrepareServiceSnapshotResult.IsOk());

			FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshotResult = CategoryInstance2.CommitServiceSnapshot();
			CHECK(CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Num() == 1);
			CHECK(CommitServiceSnapshotResult.ClientChanges.ChangedAttributes.IsEmpty());
			CHECK(CommitServiceSnapshotResult.ClientChanges.RemovedAttributes.IsEmpty());
			FSchemaVariant* AddedAttribute = CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Find(AttributeId2);
			REQUIRE(AddedAttribute);
			CHECK(*AddedAttribute == StartingVariantData);
		}

		// Check attribute was changed.
		{
			// Write value to instance 1.
			FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
			ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, ChangedVariantData } });
			TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
			CHECK(PrepareClientChangesResult.IsOk());

			// Grab service output.
			FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
			REQUIRE(SchemaServiceAttribute2Data);

			// Feed service output into instance 2.
			FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
			PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
				{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value } });
			TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
			REQUIRE(PrepareServiceSnapshotResult.IsOk());

			FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshotResult = CategoryInstance2.CommitServiceSnapshot();
			CHECK(CommitServiceSnapshotResult.ClientChanges.AddedAttributes.IsEmpty());
			CHECK(CommitServiceSnapshotResult.ClientChanges.ChangedAttributes.Num() == 1);
			CHECK(CommitServiceSnapshotResult.ClientChanges.RemovedAttributes.IsEmpty());
			TPair<FSchemaVariant, FSchemaVariant>* ChangedAttribute = CommitServiceSnapshotResult.ClientChanges.ChangedAttributes.Find(AttributeId2);
			REQUIRE(ChangedAttribute);
			CHECK(ChangedAttribute->Get<0>() == StartingVariantData);
			CHECK(ChangedAttribute->Get<1>() == ChangedVariantData);
		}

		// Check attribute was removed.
		{
			// Write value to instance 1.
			FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
			ClientChangesParams.ClientChanges.RemovedAttributes = TSet<FSchemaAttributeId>({ AttributeId2 });
			TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
			CHECK(PrepareClientChangesResult.IsOk());

			// Verify service output.
			CHECK(PrepareClientChangesResult.GetOkValue().ServiceChanges.RemovedAttributes.Contains(ServiceAttributeId2));

			// Feed service output into instance 2.
			FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
			TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
			REQUIRE(PrepareServiceSnapshotResult.IsOk());

			FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshotResult = CategoryInstance2.CommitServiceSnapshot();
			CHECK(CommitServiceSnapshotResult.ClientChanges.AddedAttributes.IsEmpty());
			CHECK(CommitServiceSnapshotResult.ClientChanges.ChangedAttributes.IsEmpty());
			CHECK(CommitServiceSnapshotResult.ClientChanges.RemovedAttributes.Num() == 1);
			CHECK(CommitServiceSnapshotResult.ClientChanges.RemovedAttributes.Contains(AttributeId2));
		}
	}

	SECTION("Valid instance - backing service - Read with different SchemaCompatibilityId - same schema with changed attribute")
	{
		TSharedRef<FSchemaRegistry> SchemaRegistry1 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 30 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry1->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry1);
		REQUIRE(CategoryInstance1.IsValid());

		TSharedRef<FSchemaRegistry> SchemaRegistry2 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 32 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry2->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance2(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry2);
		REQUIRE(CategoryInstance2.IsValid());

		// Write value to instance 1.
		FSchemaVariant VariantData(TEXT("Valid string"));
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, VariantData } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(PrepareClientChangesResult.IsOk());

		// Grab service output.
		FSchemaServiceAttributeData* SchemaServiceAttribute1Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttribute1Data);
		FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
		REQUIRE(SchemaServiceAttribute2Data);

		// Feed service output into instance 2.
		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
			{ SchemaServiceAttribute1Data->Id, SchemaServiceAttribute1Data->Value },
			{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value } });
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));

		REQUIRE(!PrepareServiceSnapshotResult.IsOk());
		CHECK(PrepareServiceSnapshotResult.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Read unknown schema")
	{
		TSharedRef<FSchemaRegistry> SchemaRegistry1 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 32 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry1->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry1);
		REQUIRE(CategoryInstance1.IsValid());

		TSharedRef<FSchemaRegistry> SchemaRegistry2 = MakeShared<FSchemaRegistry>();
		{
			FSchemaRegistryDescriptorConfig SchemaConfig;

			SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
			SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

			SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
			SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
			SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::String, { ESchemaAttributeFlags::Private }, 0, 32 });

			SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId3 } });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
			SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId3, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

			REQUIRE(SchemaRegistry2->ParseConfig(SchemaConfig));
		}

		FSchemaCategoryInstance CategoryInstance2(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry2);
		REQUIRE(CategoryInstance2.IsValid());

		// Write value to instance 1.
		FSchemaVariant VariantData(TEXT("Valid string"));
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, VariantData } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(PrepareClientChangesResult.IsOk());

		// Grab service output.
		FSchemaServiceAttributeData* SchemaServiceAttribute1Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttribute1Data);
		FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
		REQUIRE(SchemaServiceAttribute2Data);

		// Feed service output into instance 2.
		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
			{ SchemaServiceAttribute1Data->Id, SchemaServiceAttribute1Data->Value },
			{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value }});
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));

		REQUIRE(!PrepareServiceSnapshotResult.IsOk());
		CHECK(PrepareServiceSnapshotResult.GetErrorValue() == Errors::InvalidParams());
	}

	SECTION("Valid instance - backing service - Read unknown service attribute")
	{
		FSchemaRegistryDescriptorConfig SchemaConfig;
		SchemaConfig.SchemaDescriptors.Add({ BaseSchemaId1, FSchemaId(), { CategoryId1 } });
		SchemaConfig.SchemaCategoryDescriptors.Add({ CategoryId1, ServiceDescriptorId });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ BaseSchemaId1, CategoryId1, { AttributeId1 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId1, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Public, ESchemaAttributeFlags::SchemaCompatibilityId }, 0, 0 });

		SchemaConfig.SchemaDescriptors.Add({ SchemaId1, BaseSchemaId1 });
		SchemaConfig.SchemaCategoryAttributeDescriptors.Add({ SchemaId1, CategoryId1, { AttributeId2 } });
		SchemaConfig.SchemaAttributeDescriptors.Add({ AttributeId2, ESchemaAttributeType::Int64, { ESchemaAttributeFlags::Private }, 0, 0 });

		SchemaConfig.ServiceDescriptors.Add({ ServiceDescriptorId, { ServiceAttributeId1, ServiceAttributeId2 } });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId1, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });
		SchemaConfig.ServiceAttributeDescriptors.Add({ ServiceAttributeId2, AllSupportedServiceAttributeTypes, AllSupportedServiceAttributeFlags, 32 });

		TSharedRef<FSchemaRegistry> SchemaRegistry = MakeShared<FSchemaRegistry>();
		REQUIRE(SchemaRegistry->ParseConfig(SchemaConfig));

		FSchemaCategoryInstance CategoryInstance1(SchemaId1, BaseSchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance1.IsValid());

		FSchemaCategoryInstance CategoryInstance2(FSchemaId(), BaseSchemaId1, CategoryId1, SchemaRegistry);
		REQUIRE(CategoryInstance2.IsValid());

		// Write value to instance 1.
		FSchemaVariant VariantData(42ll);
		FSchemaCategoryInstancePrepareClientChanges::Params ClientChangesParams;
		ClientChangesParams.ClientChanges.UpdatedAttributes = TMap<FSchemaAttributeId, FSchemaVariant>({ { AttributeId2, VariantData } });
		TOnlineResult<FSchemaCategoryInstancePrepareClientChanges> PrepareClientChangesResult = CategoryInstance1.PrepareClientChanges(MoveTemp(ClientChangesParams));
		CHECK(PrepareClientChangesResult.IsOk());

		// Grab service output.
		FSchemaServiceAttributeData* SchemaServiceAttribute1Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId1);
		REQUIRE(SchemaServiceAttribute1Data);
		FSchemaServiceAttributeData* SchemaServiceAttribute2Data = PrepareClientChangesResult.GetOkValue().ServiceChanges.UpdatedAttributes.Find(ServiceAttributeId2);
		REQUIRE(SchemaServiceAttribute2Data);

		FSchemaServiceAttributeData UnexpectedSchemaServiceAttributeData;
		UnexpectedSchemaServiceAttributeData.Id = "UnexpectedAttributeId";
		UnexpectedSchemaServiceAttributeData.Value = TEXT("Yams");

		// Feed service output into instance 2, but include a third unexpected attribute.
		FSchemaCategoryInstancePrepareServiceSnapshot::Params PrepareServiceSnapshotParams;
		PrepareServiceSnapshotParams.ServiceSnapshot.Attributes = TMap<FSchemaServiceAttributeId, FSchemaVariant>({
			{ SchemaServiceAttribute1Data->Id, SchemaServiceAttribute1Data->Value },
			{ SchemaServiceAttribute2Data->Id, SchemaServiceAttribute2Data->Value },
			{ UnexpectedSchemaServiceAttributeData.Id, UnexpectedSchemaServiceAttributeData.Value }});
		TOnlineResult<FSchemaCategoryInstancePrepareServiceSnapshot> PrepareServiceSnapshotResult = CategoryInstance2.PrepareServiceSnapshot(MoveTemp(PrepareServiceSnapshotParams));
		REQUIRE(PrepareServiceSnapshotResult.IsOk());

		// Check that the schema in instance 2 changed from BaseSchemaId1 to SchemaId1.
		FSchemaCategoryInstanceCommitServiceSnapshot::Result CommitServiceSnapshotResult = CategoryInstance2.CommitServiceSnapshot();
		REQUIRE(CommitServiceSnapshotResult.ClientChanges.SchemaId.IsSet());
		CHECK(*CommitServiceSnapshotResult.ClientChanges.SchemaId == SchemaId1);
		REQUIRE(CategoryInstance2.GetDerivedDefinition() != nullptr);
		CHECK(CategoryInstance2.GetDerivedDefinition()->Id == SchemaId1);

		// Check that attribute data defined in SchemaId1 is received in instance 2.
		CHECK(CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Num() == 1);
		FSchemaVariant* SchemaAttributeData = CommitServiceSnapshotResult.ClientChanges.AddedAttributes.Find(AttributeId2);
		REQUIRE(SchemaAttributeData != nullptr);
		CHECK(*SchemaAttributeData == VariantData);
	}
}
