// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/AddRemoveSessionMemberHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG SESSIONS_TAG "[addremovesessionmember]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call AddSessionMember with an invalid account id, I get an error", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->LocalAccountId = FAccountId();
	AddSessionMemberHelperParams.ExpectedError = TOnlineResult<FAddSessionMember>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call AddSessionMember with an empty session name, I get an error", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FAccountId AccountId;

	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->SessionName = TEXT("");
	AddSessionMemberHelperParams.ExpectedError = TOnlineResult<FAddSessionMember>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	
	AddSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call AddSessionMember with an unregistered session name, I get an error", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FAccountId AccountId;

	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.ExpectedError = TOnlineResult<FAddSessionMember>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	AddSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;
	AddSessionMemberHelperParams.OpParams->SessionName = TEXT("UnregisteredName");

	LoginPipeline
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call AddSessionMember with valid data, the operation completes successfully", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = FName(TEXT("SessionValidNameAddMember"));
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = FName(TEXT("SchemaName"));
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->SessionName = FName(TEXT("SessionValidNameAddMember"));

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionValidNameAddMember");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	AddSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RemoveSessionMember with an invalid account id, I get an error", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FRemoveSessionMember::Params OpRemoveParams;
	FRemoveSessionMemberHelper::FHelperParams RemoveSessionMemberHelperParams;
	RemoveSessionMemberHelperParams.OpParams = &OpRemoveParams;
	RemoveSessionMemberHelperParams.OpParams->LocalAccountId = FAccountId();
	RemoveSessionMemberHelperParams.ExpectedError = TOnlineResult<FRemoveSessionMember>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FRemoveSessionMemberHelper>(MoveTemp(RemoveSessionMemberHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RemoveSessionMember with an empty session name, I get an error", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FAccountId AccountId;

	FRemoveSessionMember::Params OpRemoveParams;
	FRemoveSessionMemberHelper::FHelperParams RemoveSessionMemberHelperParams;
	RemoveSessionMemberHelperParams.OpParams = &OpRemoveParams;
	RemoveSessionMemberHelperParams.OpParams->SessionName = TEXT("");
	RemoveSessionMemberHelperParams.ExpectedError = TOnlineResult<FRemoveSessionMember>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	RemoveSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FRemoveSessionMemberHelper>(MoveTemp(RemoveSessionMemberHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RemoveSessionMember with an unregistered session name, I get an error", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	FAccountId AccountId;

	FRemoveSessionMember::Params OpRemoveParams;
	FRemoveSessionMemberHelper::FHelperParams RemoveSessionMemberHelperParams;
	RemoveSessionMemberHelperParams.OpParams = &OpRemoveParams;
	RemoveSessionMemberHelperParams.OpParams->SessionName = TEXT("UnregisteredName");
	RemoveSessionMemberHelperParams.ExpectedError = TOnlineResult<FRemoveSessionMember>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	RemoveSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FRemoveSessionMemberHelper>(MoveTemp(RemoveSessionMemberHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RemoveSessionMember with valid data, the operation completes successfully", EG_SESSIONS_ADDREMOVESESSIONMEMBER_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNameValidRemoveMember");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = FName(TEXT("SchemaName4"));
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->SessionName = TEXT("SessionNameValidRemoveMember");

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNameValidRemoveMember");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FRemoveSessionMember::Params OpRemoveParams;
	FRemoveSessionMemberHelper::FHelperParams RemoveSessionMemberHelperParams;
	RemoveSessionMemberHelperParams.OpParams = &OpRemoveParams;
	RemoveSessionMemberHelperParams.OpParams->SessionName = TEXT("SessionNameValidRemoveMember");

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	AddSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;
	RemoveSessionMemberHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams))
		.EmplaceStep<FRemoveSessionMemberHelper>(MoveTemp(RemoveSessionMemberHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}