// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Online/OnlineServicesLog.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_CREATESESSION_TAG SESSIONS_TAG "[createsession]"
#define EG_SESSIONS_CREATESESSIONEOS_TAG SESSIONS_TAG "[createsession][.EOS]"
#define EG_SESSIONS_CREATESESSIONNULL_TAG SESSIONS_TAG "[createsession][.NULL]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call CreateSession with an invalid account id, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	FCreateSession::Params OpCreateParams; 
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->LocalAccountId = FAccountId();
	CreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with an empty session name, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("");
	CreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with an empty schema name in settings, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	FAccountId AccountId;
	
	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("");
	CreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with an invalid max connections number in settings, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("MaxConnectionsSessionName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("MaxConnectionsSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 0;
	CreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with an empty custom setting name in settings, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	CreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(FName(), FCustomSessionSetting{ FSchemaVariant(false), ESchemaAttributeVisibility::Public });
	CreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with session override id less than 16 chars or more than 64, I get an error", EG_SESSIONS_CREATESESSIONEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNamePresenceEnabled1");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionIdOverride = TEXT("SessionId");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	FirstCreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNamePresenceEnabled2");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	SecondCreateSessionHelperParams.OpParams->SessionIdOverride = TEXT("SessionIdOverrideSessionIdOverrideSessionIdOverrideSessionIdOverride");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	SecondCreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with an name already in use, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	OpCreateParams.SessionName = TEXT("SessionNameInUse");
	OpCreateParams.SessionSettings.SchemaName = TEXT("SchemaName");
	OpCreateParams.SessionSettings.NumMaxConnections = 2;
	OpCreateParams.SessionSettings.bAllowNewMembers = false;

	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpCreateParams;

	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpCreateParams;
	SecondCreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidState());

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNameInUse");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));


	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with the presence flag set after there is already a presence session, I get an error", EG_SESSIONS_CREATESESSION_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNamePresenceEnabled1");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNamePresenceEnabled2");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	SecondCreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::InvalidState());

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNamePresenceEnabled1");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession with valid data, the operation completes successfully", EG_SESSIONS_CREATESESSION_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNameValid");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	
	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNameValid");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call CreateSession twice for NULL, I get an error", EG_SESSIONS_CREATESESSIONNULL_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("CreateSessionTwiceName");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("CreateSessionTwiceName1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	SecondCreateSessionHelperParams.ExpectedError = TOnlineResult<FCreateSession>(Errors::AlreadyPending());

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("CreateSessionTwiceName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}