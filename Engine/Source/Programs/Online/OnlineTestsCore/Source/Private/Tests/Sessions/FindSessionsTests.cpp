// Copyright Epic Games, Inc. All Rights Reserved.


#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/AddRemoveSessionMemberHelper.h"
#include "Helpers/Sessions/FindSessionsHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Helpers/TickForTime.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Online/OnlineServicesLog.h"
#include "EOSShared.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_FINDSESSIONS_TAG SESSIONS_TAG "[findsessions]"
#define EG_SESSIONS_FINDSESSIONSEOS_TAG SESSIONS_TAG "[findsessions][.EOS]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call FindSessions with an invalid account id, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->LocalAccountId = FAccountId();
	FindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with an invalid max results number, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FAccountId AccountId;

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 0;
	FindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FindSessionsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with an empty custom setting name in a filter, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FAccountId AccountId;

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 10;
	FindSessionsHelperParams.OpParams->Filters.Add(FFindSessionsSearchFilter{ FName(), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(false) });
	FindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FindSessionsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with an invalid session id as filter, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FAccountId AccountId;

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 10;
	FindSessionsHelperParams.OpParams->SessionId = FOnlineSessionId();
	FindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	
	FindSessionsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with an invalid account id as filter, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FAccountId AccountId;

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 10;
	FindSessionsHelperParams.OpParams->TargetUser = FAccountId();
	FindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	
	FindSessionsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with an empty search filter key, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FAccountId AccountId;

	FFindSessions::Params OpFirstParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFirstParams;
	FindSessionsHelperParams.OpParams->MaxResults = 3;
	FindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName(""), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(int64(64)) });
	FindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FindSessionsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions while a search is already running for that user, I get an error", EG_SESSIONS_FINDSESSIONS_TAG)
{
#if ONLINETESTS_USEEXTERNAUTH
	ReturnAccounts();
#endif

	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("AlreadyPendingName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("AlreadyPendingSchema");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FFindSessions::Params OpFirstFindParams;
	FFindSessionsHelper::FHelperParams FindFirstSessionsHelperParams;
	FindFirstSessionsHelperParams.OpParams = &OpFirstFindParams;
	FindFirstSessionsHelperParams.OpParams->MaxResults = 1;

	FFindSessions::Params OpSecondFindParams;
	FFindSessionsHelper::FHelperParams FindSecondSessionsHelperParams;
	FindSecondSessionsHelperParams.OpParams = &OpSecondFindParams;
	FindSecondSessionsHelperParams.OpParams->MaxResults = 1;
	FindSecondSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::AlreadyPending());

	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->SessionName = TEXT("AlreadyPendingName");

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("AlreadyPendingName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	AddSessionMemberHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FindFirstSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	FindFirstSessionsHelperParams.OpParams->TargetUser = FirstAccountId;

	FindSecondSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	FindSecondSessionsHelperParams.OpParams->TargetUser = FirstAccountId;

	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams))
		.EmplaceAsyncLambda([&FindFirstSessionsHelperParams, &FindSecondSessionsHelperParams](FAsyncLambdaResult Promise, const IOnlineServicesPtr& OnlineServices)
			{
				ISessionsPtr SessionsInterface = OnlineServices->GetSessionsInterface();

				SessionsInterface->FindSessions(MoveTemp(*FindFirstSessionsHelperParams.OpParams))
					.OnComplete([Promise = MoveTemp(Promise)](const TOnlineResult<FFindSessions>& Result)
						{
							CHECK_OP(Result);
							Promise->SetValue(true);
						});

				SessionsInterface->FindSessions(MoveTemp(*FindSecondSessionsHelperParams.OpParams))
					.OnComplete([&FindSecondSessionsHelperParams](const TOnlineResult<FFindSessions>& Result)
						{
							CHECK(Result.IsError());
							REQUIRE(Result.GetErrorValue() == FindSecondSessionsHelperParams.ExpectedError->GetErrorValue());
						});
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with valid information, I can get an empty list if no sessions are found", EG_SESSIONS_FINDSESSIONS_TAG)
{
	FAccountId AccountId;

	FFindSessions::Params OpFirstParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFirstParams;
	FindSessionsHelperParams.OpParams->MaxResults = 3;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FindSessionsHelperParams.OpParams->LocalAccountId = AccountId;
	FindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("NonExistentSettingName"), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(int64(64)) });

	LoginPipeline
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with valid information, I will still not find sessions which JoinPolicy is not public", EG_SESSIONS_FINDSESSIONS_TAG)
{
#if ONLINETESTS_USEEXTERNAUTH
	ReturnAccounts();
#endif
	
	FAccountId FirstAccountId, SecondAccountId;
	
	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("FriendsOnlyName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("FriendsOnlySchema");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->SessionSettings.JoinPolicy = ESessionJoinPolicy::FriendsOnly;
	
	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 1;
	
	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->SessionName = TEXT("FriendsOnlyName");
	
	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("FriendsOnlyName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;
	
	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });
	
	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	
	AddSessionMemberHelperParams.OpParams->LocalAccountId = FirstAccountId;
	
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	
	FindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	FindSessionsHelperParams.OpParams->TargetUser = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call FindSessions with valid information, I will get a list of ids for all the valid sessions", EG_SESSIONS_FINDSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("ValidName");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("ValidNameSchema");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("ValidName1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("ValidNameSchema1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FAddSessionMember::Params OpFirstAddParams;
	FAddSessionMemberHelper::FHelperParams FirstAddSessionMemberHelperParams;
	FirstAddSessionMemberHelperParams.OpParams = &OpFirstAddParams;
	FirstAddSessionMemberHelperParams.OpParams->SessionName = TEXT("ValidName");

	FAddSessionMember::Params OpSecondAddParams;
	FAddSessionMemberHelper::FHelperParams SecondAddSessionMemberHelperParams;
	SecondAddSessionMemberHelperParams.OpParams = &OpSecondAddParams;
	SecondAddSessionMemberHelperParams.OpParams->SessionName = TEXT("ValidName1");

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 2;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("ValidName");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("ValidName1");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstAddSessionMemberHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondAddSessionMemberHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	FindSessionsHelperParams.OpParams->TargetUser = FirstAccountId;

	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionsFound = 2;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(6000))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(FirstAddSessionMemberHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(SecondAddSessionMemberHelperParams))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("Call FindSessions using custom setting as a filter Bool", EG_SESSIONS_FINDSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameBoolFilter");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(FName("BoolSettingName"), FCustomSessionSetting{ FSchemaVariant(true), ESchemaAttributeVisibility::Public });

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameBoolFilter1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;

	FFindSessions::Params OpFirstFindParams;
	FFindSessionsHelper::FHelperParams FirstFindSessionsHelperParams;
	FirstFindSessionsHelperParams.OpParams = &OpFirstFindParams;
	FirstFindSessionsHelperParams.OpParams->MaxResults = 1;
	FirstFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("BoolSettingName"), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(true) });
	
	FFindSessions::Params OpSecondFindParams;
	FFindSessionsHelper::FHelperParams SecondFindSessionsHelperParams;
	SecondFindSessionsHelperParams.OpParams = &OpSecondFindParams;
	SecondFindSessionsHelperParams.OpParams->MaxResults = 1;
	SecondFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("BoolSettingName"), ESchemaAttributeComparisonOp::NotEquals, FSchemaVariant(true) });

	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameBoolFilter");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameBoolFilter1");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
	.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
	.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
	.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(6000))
	.EmplaceStep<FFindSessionsHelper>(MoveTemp(FirstFindSessionsHelperParams), ExpectedSessionsFound)
	.EmplaceStep<FFindSessionsHelper>(MoveTemp(SecondFindSessionsHelperParams))
	.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
	.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("Call FindSessions using custom setting as a filter Int", EG_SESSIONS_FINDSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameIntFilter");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(FName("Int64SettingName"), FCustomSessionSetting{ FSchemaVariant(int64(64)), ESchemaAttributeVisibility::Public });

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameIntFilter1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;

	FFindSessions::Params OpThirdFindParams;
	FFindSessionsHelper::FHelperParams ThirdFindSessionsHelperParams;
	ThirdFindSessionsHelperParams.OpParams = &OpThirdFindParams;
	ThirdFindSessionsHelperParams.OpParams->MaxResults = 1;
	ThirdFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("Int64SettingName"), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(int64(64)) });

	FFindSessions::Params OpFourthFindParams;
	FFindSessionsHelper::FHelperParams FourthFindSessionsHelperParams;
	FourthFindSessionsHelperParams.OpParams = &OpFourthFindParams;
	FourthFindSessionsHelperParams.OpParams->MaxResults = 1;
	FourthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("Int64SettingName"), ESchemaAttributeComparisonOp::NotEquals, FSchemaVariant(int64(5585)) });

	FFindSessions::Params OpFifthFindParams;
	FFindSessionsHelper::FHelperParams FifthFindSessionsHelperParams;
	FifthFindSessionsHelperParams.OpParams = &OpFifthFindParams;
	FifthFindSessionsHelperParams.OpParams->MaxResults = 1;
	FifthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("Int64SettingName"), ESchemaAttributeComparisonOp::GreaterThan, FSchemaVariant(int64(66)) });

	FFindSessions::Params OpSixthFindParams;
	FFindSessionsHelper::FHelperParams SixthFindSessionsHelperParams;
	SixthFindSessionsHelperParams.OpParams = &OpSixthFindParams;
	SixthFindSessionsHelperParams.OpParams->MaxResults = 1;
	SixthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("Int64SettingName"), ESchemaAttributeComparisonOp::GreaterThanEquals, FSchemaVariant(int64(63)) });

	FFindSessions::Params OpSeventhFindParams;
	FFindSessionsHelper::FHelperParams SeventhFindSessionsHelperParams;
	SeventhFindSessionsHelperParams.OpParams = &OpSeventhFindParams;
	SeventhFindSessionsHelperParams.OpParams->MaxResults = 1;
	SeventhFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("Int64SettingName"), ESchemaAttributeComparisonOp::LessThan, FSchemaVariant(int64(65)) });

	FFindSessions::Params OpEighthFindParams;
	FFindSessionsHelper::FHelperParams EighthFindSessionsHelperParams;
	EighthFindSessionsHelperParams.OpParams = &OpEighthFindParams;
	EighthFindSessionsHelperParams.OpParams->MaxResults = 1;
	EighthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("Int64SettingName"), ESchemaAttributeComparisonOp::LessThanEquals, FSchemaVariant(int64(66)) });

	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameIntFilter");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameIntFilter1");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	ThirdFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	FourthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	FifthFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	SixthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;	
	SeventhFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	EighthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(6000))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(ThirdFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FourthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FifthFindSessionsHelperParams))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(SixthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(SeventhFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(EighthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("Call FindSessions using custom setting as a filter Double", EG_SESSIONS_FINDSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameDoubleFilter");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	FSchemaVariant DoubleSettingValue = 100.0;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(FName("DoubleSettingName"), FCustomSessionSetting{ FSchemaVariant(DoubleSettingValue), ESchemaAttributeVisibility::Public });

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameDoubleFilter1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(FName("DoubleSettingName"), FCustomSessionSetting{ FSchemaVariant(10.0), ESchemaAttributeVisibility::Public });

	FFindSessions::Params OpNinthFindParams;
	FFindSessionsHelper::FHelperParams NinthFindSessionsHelperParams;
	NinthFindSessionsHelperParams.OpParams = &OpNinthFindParams;
	NinthFindSessionsHelperParams.OpParams->MaxResults = 1;
	NinthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(100.0) });

	FFindSessions::Params OpTenthFindParams;
	FFindSessionsHelper::FHelperParams TenthFindSessionsHelperParams;
	TenthFindSessionsHelperParams.OpParams = &OpTenthFindParams;
	TenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	TenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::NotEquals, FSchemaVariant(10.56) });

	FFindSessions::Params OpEleventhFindParams;
	FFindSessionsHelper::FHelperParams EleventhFindSessionsHelperParams;
	EleventhFindSessionsHelperParams.OpParams = &OpEleventhFindParams;
	EleventhFindSessionsHelperParams.OpParams->MaxResults = 1;
	EleventhFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::GreaterThan, FSchemaVariant(9.56) });

	FFindSessions::Params OpTwelfthFindParams;
	FFindSessionsHelper::FHelperParams TwelfthFindSessionsHelperParams;
	TwelfthFindSessionsHelperParams.OpParams = &OpTwelfthFindParams;
	TwelfthFindSessionsHelperParams.OpParams->MaxResults = 1;
	TwelfthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::GreaterThanEquals, FSchemaVariant(9.1345) });

	FFindSessions::Params OpThirteenthFindParams;
	FFindSessionsHelper::FHelperParams ThirteenthFindSessionsHelperParams;
	ThirteenthFindSessionsHelperParams.OpParams = &OpThirteenthFindParams;
	ThirteenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	ThirteenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::LessThan, FSchemaVariant(111.16) });

	FFindSessions::Params OpFourteenthFindParams;
	FFindSessionsHelper::FHelperParams FourteenthFindSessionsHelperParams;
	FourteenthFindSessionsHelperParams.OpParams = &OpFourteenthFindParams;
	FourteenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	FourteenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::LessThanEquals, FSchemaVariant(100.0) });

	FFindSessions::Params OpFifteenthFindParams;
	FFindSessionsHelper::FHelperParams FifteenthFindSessionsHelperParams;
	FifteenthFindSessionsHelperParams.OpParams = &OpFifteenthFindParams;
	FifteenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	FifteenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("DoubleSettingName"), ESchemaAttributeComparisonOp::Near, FSchemaVariant(90.0) });

	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameDoubleFilter");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameDoubleFilter1");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	NinthFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	TenthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	TwelfthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	EleventhFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	ThirteenthFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	FourteenthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	FifteenthFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(6000))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(NinthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(TenthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(TwelfthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(EleventhFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(ThirteenthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FourteenthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceAsyncLambda([&FifteenthFindSessionsHelperParams, &DoubleSettingValue](FAsyncLambdaResult Promise, const IOnlineServicesPtr& OnlineServices)
			{
				ISessionsPtr SessionsInterface = OnlineServices->GetSessionsInterface();
				SessionsInterface->FindSessions(MoveTemp(*FifteenthFindSessionsHelperParams.OpParams))
					.OnComplete([Promise = MoveTemp(Promise), SessionsInterface, &DoubleSettingValue](const TOnlineResult<FFindSessions>& Result)
						{
							REQUIRE_OP(Result);
							REQUIRE(!Result.GetOkValue().FoundSessionIds.IsEmpty());

							FGetSessionById::Params GetSessionParams;
							GetSessionParams.SessionId = Result.GetOkValue().FoundSessionIds[0];

							const FSessionSettings& Settings = SessionsInterface->GetSessionById(MoveTemp(GetSessionParams)).GetOkValue().Session->GetSessionSettings();
							const FCustomSessionSetting* SettingValue = Settings.CustomSettings.Find(FName("DoubleSettingName"));
							CHECK(SettingValue != nullptr);
							CHECK(SettingValue->Data == DoubleSettingValue);


							Promise->SetValue(true);
						});

			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("Call FindSessions using custom setting as a filter String", EG_SESSIONS_FINDSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateSessionParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateSessionParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameStringFilter");
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	FirstCreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(FName("StringSettingName"), FCustomSessionSetting{ FSchemaVariant(TEXT("TestStringData")), ESchemaAttributeVisibility::Public });

	FCreateSession::Params OpSecondCreateSessionParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateSessionParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameStringFilter1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;

	FFindSessions::Params OpSixteenFindParams;
	FFindSessionsHelper::FHelperParams SixteenthFindSessionsHelperParams;
	SixteenthFindSessionsHelperParams.OpParams = &OpSixteenFindParams;
	SixteenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	SixteenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("StringSettingName"), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(TEXT("TestStringData")) });

	FFindSessions::Params OpSeventeenFindParams;
	FFindSessionsHelper::FHelperParams SeventeenthFindSessionsHelperParams;
	SeventeenthFindSessionsHelperParams.OpParams = &OpSeventeenFindParams;
	SeventeenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	SeventeenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("StringSettingName"), ESchemaAttributeComparisonOp::NotEquals, FSchemaVariant(TEXT("NotTestStringData")) });

	FFindSessions::Params OpEighteenthFindParams;
	FFindSessionsHelper::FHelperParams EighteenthFindSessionsHelperParams;
	EighteenthFindSessionsHelperParams.OpParams = &OpEighteenthFindParams;
	EighteenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	EighteenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("StringSettingName"), ESchemaAttributeComparisonOp::NotIn, FSchemaVariant(TEXT("NotTestStringData;OrNotOtherStringData;NotAnotherStringData")) });

	FFindSessions::Params OpNineteenthFindParams;
	FFindSessionsHelper::FHelperParams NineteenthFindSessionsHelperParams;
	NineteenthFindSessionsHelperParams.OpParams = &OpNineteenthFindParams;
	NineteenthFindSessionsHelperParams.OpParams->MaxResults = 1;
	NineteenthFindSessionsHelperParams.OpParams->Filters.Emplace(FFindSessionsSearchFilter{ FName("StringSettingName"), ESchemaAttributeComparisonOp::In, FSchemaVariant(TEXT("TestStringData;OtherStringData;AnotherStringData")) });
	NineteenthFindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::InvalidParams());
	
	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameStringFilter");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("FindSessionNameStringFilter1");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	
	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	SixteenthFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	SeventeenthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	EighteenthFindSessionsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	NineteenthFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	TOptional<uint32_t> NoSessionsFound = TOptional<uint32_t>();
	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(6000))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(SixteenthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(SeventeenthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(EighteenthFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(NineteenthFindSessionsHelperParams), NoSessionsFound, []()
			{
				UE_LOG_ONLINETESTS(Verbose, TEXT("Functionality being tested is still unsupported as of EOSSDK Version: 1.15.3-21924193"));
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}