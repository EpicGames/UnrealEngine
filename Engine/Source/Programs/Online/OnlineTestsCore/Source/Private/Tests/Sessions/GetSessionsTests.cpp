// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Helpers/Sessions/SendRejectSessionInviteHelper.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Online/OnlineServicesLog.h"
#include "EOSShared.h"
#include "Helpers/TickForTime.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_GETSESSIONS_TAG SESSIONS_TAG "[getsessions]"
#define EG_SESSIONS_GETSESSIONSEOS_TAG SESSIONS_TAG "[getsessions][.EOS]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call GetAllSessions with an invalid account id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetAllSessions::Params OpGetAllParams;
				OpGetAllParams.LocalAccountId = FAccountId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetAllSessions> Result = SessionsInterface->GetAllSessions(MoveTemp(OpGetAllParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetAllSessions before creating or joining any sessions, I get a successful result which is an empty array", EG_SESSIONS_GETSESSIONS_TAG)
{
#if ONLINETESTS_USEEXTERNAUTH
	ReturnAccounts();
#endif

	FAccountId AccountId;
	GetLoginPipeline({ AccountId })
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetAllSessions::Params OpParams;
				OpParams.LocalAccountId = AccountId;

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();				
				TOnlineResult<FGetAllSessions> Result = SessionsInterface->GetAllSessions(MoveTemp(OpParams));
				REQUIRE_OP(Result);
				CHECK(Result.GetOkValue().Sessions.IsEmpty());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetAllSessions with valid conditions, I get a valid array of session references", EG_SESSIONS_GETSESSIONS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetAllSessionsValidName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetAllSessionsValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&AccountId, &ExpectedSessionsFound](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetAllSessions::Params OpGetAllParams;
				OpGetAllParams.LocalAccountId = AccountId;

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetAllSessions> Result = SessionsInterface->GetAllSessions(MoveTemp(OpGetAllParams));
				REQUIRE_OP(Result);
				CHECK(Result.GetOkValue().Sessions.Num() == ExpectedSessionsFound);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));


	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionByName with an empty session name, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	FAccountId AccountId;

	GetLoginPipeline({ AccountId })
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetSessionByName::Params OpParams;
				OpParams.LocalName = TEXT("");

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName(MoveTemp(OpParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionByName with an unregistered session name, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	FAccountId AccountId;

	GetLoginPipeline({ AccountId })
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetSessionByName::Params OpGetByNameParams;
				OpGetByNameParams.LocalName = TEXT("GetUnregisteredSessionName");

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName(MoveTemp(OpGetByNameParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::NotFound());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionByName with valid information, it returns a valid session reference", EG_SESSIONS_GETSESSIONS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetSessionByNameValidName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetSessionByNameValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;
	
	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetSessionByName::Params OpGetByNameParams;
				OpGetByNameParams.LocalName = TEXT("GetSessionByNameValidName");

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName(MoveTemp(OpGetByNameParams));
				REQUIRE_OP(Result);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionById with an invalid session id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	FAccountId AccountId;

	GetLoginPipeline({ AccountId })
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetSessionById::Params OpGetByIdParams;
				OpGetByIdParams.SessionId = FOnlineSessionId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionById> Result = SessionsInterface->GetSessionById(MoveTemp(OpGetByIdParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionById with a valid but unregistered session id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetUnregisteredSessionByIdName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetUnregisteredSessionByIdName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FGetSessionById::Params OpGetByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&OpGetByIdParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("GetUnregisteredSessionByIdName") });
				REQUIRE_OP(Result);

				OpGetByIdParams.SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams))
		.EmplaceLambda([&OpGetByIdParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionById> Result = SessionsInterface->GetSessionById(MoveTemp(OpGetByIdParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::NotFound());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionById with a valid id for a valid session, I get a valid session reference in return", EG_SESSIONS_GETSESSIONS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetSessionByValidIdName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetSessionByValidIdName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FGetSessionById::Params OpGetByIdParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&OpGetByIdParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("GetSessionByValidIdName") });
				REQUIRE_OP(Result);

				OpGetByIdParams.SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceLambda([&OpGetByIdParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionById> Result = SessionsInterface->GetSessionById(MoveTemp(OpGetByIdParams));
				REQUIRE_OP(Result);
				CHECK(Result.GetOkValue().Session->GetSessionId() == OpGetByIdParams.SessionId);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));


	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetPresenceSession with an invalid id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetPresenceSession::Params OpGetPresenceParams;
				OpGetPresenceParams.LocalAccountId = FAccountId();
	
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetPresenceSession> Result = SessionsInterface->GetPresenceSession(MoveTemp(OpGetPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetPresenceSession with an unregistered id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	FAccountId AccountId;

	GetLoginPipeline({ AccountId })
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetPresenceSession::Params OpGetPresenceParams;
				OpGetPresenceParams.LocalAccountId = AccountId;

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetPresenceSession> Result = SessionsInterface->GetPresenceSession(MoveTemp(OpGetPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidState());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetPresenceSession with a valid id, I get a valid reference to the session", EG_SESSIONS_GETSESSIONS_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetPresenceSessionWithValidIdName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetPresenceSessionWithValidIdName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FGetPresenceSession::Params OpGetPresenceParams;
				OpGetPresenceParams.LocalAccountId = AccountId;

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetPresenceSession> Result = SessionsInterface->GetPresenceSession(MoveTemp(OpGetPresenceParams));
				REQUIRE_OP(Result);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionInviteById with an invalid account id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	FGetSessionInviteById::Params OpGetInviteByIdParams;
	OpGetInviteByIdParams.LocalAccountId = FAccountId();

	GetPipeline()
		.EmplaceLambda([&OpGetInviteByIdParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionInviteById> Result = SessionsInterface->GetSessionInviteById(MoveTemp(OpGetInviteByIdParams));

				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());

			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionInviteById with an invalid session invite id, I get an error", EG_SESSIONS_GETSESSIONS_TAG)
{
	FAccountId AccountId;

	FGetSessionInviteById::Params OpGetInviteByIdParams;
	OpGetInviteByIdParams.SessionInviteId = FSessionInviteId();
	
	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	OpGetInviteByIdParams.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceLambda([&OpGetInviteByIdParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionInviteById> Result = SessionsInterface->GetSessionInviteById(MoveTemp(OpGetInviteByIdParams));

				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());

			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionInviteById with a valid account id, but without an existing invite, I get an error", EG_SESSIONS_GETSESSIONSEOS_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetInviteByIdWithValidAccountIdName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("GetInviteByIdWithValidAccountIdName");

	FGetSessionInviteById::Params OpGetInviteByIdParams;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetInviteByIdWithValidAccountIdName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	OpGetInviteByIdParams.LocalAccountId = FirstAccountId;
	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(SecondAccountId);
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams), [&OpGetInviteByIdParams](const UE::Online::FSessionInviteId& InSessionInviteId)
			{
				OpGetInviteByIdParams.SessionInviteId = InSessionInviteId;
			})
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceLambda([&OpGetInviteByIdParams, &FirstAccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionInviteById> Result = SessionsInterface->GetSessionInviteById(MoveTemp(OpGetInviteByIdParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidState());
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));


	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionInviteById with valid invite id, but without invite, I get an error", EG_SESSIONS_GETSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetInviteByIdWithValidInviteIdName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("GetInviteByIdWithValidInviteIdName");

	FRejectSessionInvite::Params OpRejectInviteParams;
	FRejectSessionInviteHelper::FHelperParams RejectSessionInviteHelperParams;
	RejectSessionInviteHelperParams.OpParams = &OpRejectInviteParams;

	FGetSessionInviteById::Params OpGetInviteByIdParams;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetInviteByIdWithValidInviteIdName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	OpGetInviteByIdParams.LocalAccountId = SecondAccountId;
	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(SecondAccountId);
	RejectSessionInviteHelperParams.OpParams->LocalAccountId = SecondAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams), [&OpGetInviteByIdParams, &RejectSessionInviteHelperParams](const UE::Online::FSessionInviteId& InSessionInviteId)
			{
				OpGetInviteByIdParams.SessionInviteId = InSessionInviteId;
				RejectSessionInviteHelperParams.OpParams->SessionInviteId = InSessionInviteId;
			})
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FRejectSessionInviteHelper>(MoveTemp(RejectSessionInviteHelperParams))
		.EmplaceLambda([&OpGetInviteByIdParams, &FirstAccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionInviteById> Result = SessionsInterface->GetSessionInviteById(MoveTemp(OpGetInviteByIdParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::NotFound());
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));


	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call GetSessionInviteById with a valid data, I get a valid reference to the session invite", EG_SESSIONS_GETSESSIONSEOS_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("GetInviteByIdValidName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("GetInviteByIdValidName");

	FGetSessionInviteById::Params OpGetInviteByIdParams;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("GetInviteByIdValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	OpGetInviteByIdParams.LocalAccountId = SecondAccountId;
	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(SecondAccountId);
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams), [&OpGetInviteByIdParams](const UE::Online::FSessionInviteId& InSessionInviteId)
			{
				OpGetInviteByIdParams.SessionInviteId = InSessionInviteId;
			})
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceLambda([&OpGetInviteByIdParams, &FirstAccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionInviteById> Result = SessionsInterface->GetSessionInviteById(MoveTemp(OpGetInviteByIdParams));
				REQUIRE(Result.GetOkValue().SessionInvite->GetInviteId().IsValid());
				CHECK(Result.GetOkValue().SessionInvite->GetSenderId() == FirstAccountId);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}