// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/FindSessionsHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Helpers/Sessions/AddRemoveSessionMemberHelper.h"
#include "Helpers/Sessions/JoinSessionHelper.h"
#include "Helpers/Sessions/UpdateSessionSettingsHelper.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Online/OnlineServicesLog.h"
#include "EOSShared.h"
#include "Helpers/LambdaStep.h"
#include "Helpers/TickForTime.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_JOINSESSIONS_TAG SESSIONS_TAG "[joinsession]"
#define EG_SESSIONS_JOINSESSIONSEOS_TAG SESSIONS_TAG "[joinsession][.EOS]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call JoinSession with an invalid account id, I get an error", EG_SESSIONS_JOINSESSIONS_TAG)
{
	FJoinSession::Params OpJoinParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpJoinParams;
	JoinSessionHelperParams.OpParams->LocalAccountId = FAccountId();
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession with an invalid session id, I get an error", EG_SESSIONS_JOINSESSIONS_TAG)
{
	FAccountId AccountId;

	FJoinSession::Params OpJoinParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpJoinParams;
	JoinSessionHelperParams.OpParams->SessionId = FOnlineSessionId();
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	JoinSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession with an empty session name, I get an error", EG_SESSIONS_JOINSESSIONS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("EmptySessionNameJoin");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("EmptySessionSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("EmptySessionNameJoin");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FJoinSession::Params OpJoinParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpJoinParams;
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	JoinSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&JoinSessionHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("EmptySessionNameJoin") });
				REQUIRE_OP(Result);

				JoinSessionHelperParams.OpParams->SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession with an name already in use, I get an error", EG_SESSIONS_JOINSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SessionNameInUseSchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin1");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SessionNameInUseSchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;

	// Friends Only for the second session, because otherwise it defaults to public.
	// For LAN sessions, we will try to host public ones and only 1 can be hosted at a time, which should be occupied by the first session.
	SecondCreateSessionHelperParams.OpParams->SessionSettings.JoinPolicy = ESessionJoinPolicy::FriendsOnly;

	FAddSessionMember::Params OpFirstAddParams;
	FAddSessionMemberHelper::FHelperParams FirstAddSessionMemberHelperParams;
	FirstAddSessionMemberHelperParams.OpParams = &OpFirstAddParams;
	FirstAddSessionMemberHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin");

	FAddSessionMember::Params OpSecondAddParams;
	FAddSessionMemberHelper::FHelperParams SecondAddSessionMemberHelperParams;
	SecondAddSessionMemberHelperParams.OpParams = &OpSecondAddParams;
	SecondAddSessionMemberHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin1");

	FFindSessions::Params OpFirstFindParams;
	FFindSessionsHelper::FHelperParams FirstFindSessionsHelperParams;
	FirstFindSessionsHelperParams.OpParams = &OpFirstFindParams;
	FirstFindSessionsHelperParams.OpParams->MaxResults = 2;

	FFindSessions::Params OpSecondFindParams;
	FFindSessionsHelper::FHelperParams SecondFindSessionsHelperParams;
	SecondFindSessionsHelperParams.OpParams = &OpSecondFindParams;
	SecondFindSessionsHelperParams.OpParams->MaxResults = 2;

	FJoinSession::Params OpJoinParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpJoinParams;
	JoinSessionHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin");
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::InvalidState());

	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNameInUseJoin1");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstAddSessionMemberHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondAddSessionMemberHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	SecondFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	JoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::InvalidState());

	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(FirstAddSessionMemberHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(SecondAddSessionMemberHelperParams))
		.EmplaceLambda([&FirstFindSessionsHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("SessionNameInUseJoin") });
				REQUIRE_OP(Result);

				FirstFindSessionsHelperParams.OpParams->SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceLambda([&JoinSessionHelperParams, &SecondFindSessionsHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("SessionNameInUseJoin1") });
				REQUIRE_OP(Result);

				FOnlineSessionId SessionId = Result.GetOkValue().Session->GetSessionId();

				JoinSessionHelperParams.OpParams->SessionId = SessionId;
				SecondFindSessionsHelperParams.OpParams->SessionId = SessionId;

			})
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FirstFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(SecondFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession with a valid but unregistered session id, I get an error", EG_SESSIONS_JOINSESSIONSEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionIdNotFoundJoinName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("IdNotFoundSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FFindSessions::Params OpFirstFindParams;
	FFindSessionsHelper::FHelperParams FirstFindSessionsHelperParams;
	FirstFindSessionsHelperParams.OpParams = &OpFirstFindParams;
	FirstFindSessionsHelperParams.OpParams->MaxResults = 1;

	FFindSessions::Params OpSecondFindParams;
	FFindSessionsHelper::FHelperParams SecondFindSessionsHelperParams;
	SecondFindSessionsHelperParams.OpParams = &OpSecondFindParams;
	SecondFindSessionsHelperParams.OpParams->MaxResults = 1;
	SecondFindSessionsHelperParams.ExpectedError = TOnlineResult<FFindSessions>(Errors::NotFound());

	FJoinSession::Params OpFirstJoinParams;
	FJoinSessionHelper::FHelperParams FirstJoinSessionHelperParams;
	FirstJoinSessionHelperParams.OpParams = &OpFirstJoinParams;
	FirstJoinSessionHelperParams.OpParams->SessionName = TEXT("SessionIdNotFoundJoinName2");
	FirstJoinSessionHelperParams.OpParams->bPresenceEnabled = true;

	FJoinSession::Params OpSecondJoinParams;
	FJoinSessionHelper::FHelperParams SecondJoinSessionHelperParams;
	SecondJoinSessionHelperParams.OpParams = &OpSecondJoinParams;
	SecondJoinSessionHelperParams.OpParams->SessionName = TEXT("SessionIdNotFoundJoinName3");
	SecondJoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::NotFound());

	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionIdNotFoundJoinName");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionIdNotFoundJoinName");
	SecondLeaveSessionHelperParams.ExpectedError = TOnlineResult<FLeaveSession>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FirstFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	SecondFindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	FirstJoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	SecondJoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;

	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&FirstJoinSessionHelperParams, &SecondJoinSessionHelperParams, &FirstFindSessionsHelperParams, &SecondFindSessionsHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("SessionIdNotFoundJoinName") });
				REQUIRE_OP(Result);

				FOnlineSessionId SessionId = Result.GetOkValue().Session->GetSessionId();

				FirstFindSessionsHelperParams.OpParams->SessionId = SessionId;
				SecondFindSessionsHelperParams.OpParams->SessionId = SessionId;

				FirstJoinSessionHelperParams.OpParams->SessionId = SessionId;
				SecondJoinSessionHelperParams.OpParams->SessionId = SessionId;
			})
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FirstFindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(FirstJoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(SecondFindSessionsHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams))
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(SecondJoinSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession for a session I'm already registered in, I get an error", EG_SESSIONS_JOINSESSIONSEOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("AlreadyInSessionJoinName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("AlreadyInSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FAddSessionMember::Params OpAddParams;
	FAddSessionMemberHelper::FHelperParams AddSessionMemberHelperParams;
	AddSessionMemberHelperParams.OpParams = &OpAddParams;
	AddSessionMemberHelperParams.OpParams->SessionName = TEXT("AlreadyInSessionJoinName");

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 1;

	FJoinSession::Params OpFirstJoinParams;
	FJoinSessionHelper::FHelperParams FirstJoinSessionHelperParams;
	FirstJoinSessionHelperParams.OpParams = &OpFirstJoinParams;
	FirstJoinSessionHelperParams.OpParams->SessionName = TEXT("AlreadyInSessionJoinName2");

	FJoinSession::Params OpSecondJoinParams;
	FJoinSessionHelper::FHelperParams SecondJoinSessionHelperParams;
	SecondJoinSessionHelperParams.OpParams = &OpSecondJoinParams;
	SecondJoinSessionHelperParams.OpParams->SessionName = TEXT("AlreadyInSessionJoinName3");
	SecondJoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::AccessDenied());

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("AlreadyInSessionJoinName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	AddSessionMemberHelperParams.OpParams->LocalAccountId = SecondAccountId;

	FirstJoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	SecondJoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	FindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&FirstJoinSessionHelperParams, &SecondJoinSessionHelperParams, &FindSessionsHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("AlreadyInSessionJoinName") });
				REQUIRE_OP(Result);

				FOnlineSessionId SessionId = Result.GetOkValue().Session->GetSessionId();

				FirstJoinSessionHelperParams.OpParams->SessionId = SessionId;
				SecondJoinSessionHelperParams.OpParams->SessionId = SessionId;
				FindSessionsHelperParams.OpParams->SessionId = SessionId;
			})
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(FirstJoinSessionHelperParams))
		.EmplaceStep<FAddSessionMemberHelper>(MoveTemp(AddSessionMemberHelperParams))
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(SecondJoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession for a session that is not joinable, I get an error", EG_SESSIONS_JOINSESSIONSEOS_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionNotJoinableName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("NotJoinableSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->SessionSettings.bAllowNewMembers = true;

	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("SessionNotJoinableName");
	UpdateSessionSettingsHelperParams.OpParams->Mutations.bAllowNewMembers = false;
	UpdateSessionSettingsHelperParams.OpParams->Mutations.JoinPolicy = ESessionJoinPolicy::InviteOnly;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionNotJoinableName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 1;

	FJoinSession::Params OpJoinParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpJoinParams;
	JoinSessionHelperParams.OpParams->SessionName = TEXT("SessionNotJoinableName1");
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::AccessDenied());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = FirstAccountId;
	FindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;
	JoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&FindSessionsHelperParams, &JoinSessionHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("SessionNotJoinableName") });
				REQUIRE_OP(Result);

				FOnlineSessionId SessionId = Result.GetOkValue().Session->GetSessionId();

				FindSessionsHelperParams.OpParams->SessionId = SessionId;
				JoinSessionHelperParams.OpParams->SessionId = SessionId;
			})
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams))
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession with presence enabled when there is already a presence session, I get an error", EG_SESSIONS_JOINSESSIONS_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("PresenceJoinName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("PresenceJoinSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->SessionSettings.bAllowNewMembers = true;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("PresenceJoinName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FJoinSession::Params OpJoinParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpJoinParams;
	JoinSessionHelperParams.OpParams->SessionName = TEXT("PresenceJoinName2");
	JoinSessionHelperParams.OpParams->bPresenceEnabled = true;
	JoinSessionHelperParams.ExpectedError = TOnlineResult<FJoinSession>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;
	JoinSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&JoinSessionHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("PresenceJoinName") });
				REQUIRE_OP(Result);

				JoinSessionHelperParams.OpParams->SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call JoinSession with valid information, the operation completes successfully", EG_SESSIONS_JOINSESSIONSEOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionValidJoinName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("ValidJoinSchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;

	FFindSessions::Params OpFindParams;
	FFindSessionsHelper::FHelperParams FindSessionsHelperParams;
	FindSessionsHelperParams.OpParams = &OpFindParams;
	FindSessionsHelperParams.OpParams->MaxResults = 10;

	FJoinSession::Params OpParams;
	FJoinSessionHelper::FHelperParams JoinSessionHelperParams;
	JoinSessionHelperParams.OpParams = &OpParams;
	JoinSessionHelperParams.OpParams->SessionName = TEXT("SessionValidJoinName2");

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionValidJoinName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	FindSessionsHelperParams.OpParams->LocalAccountId = SecondAccountId;

	JoinSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionsFound = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&FindSessionsHelperParams, &JoinSessionHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("SessionValidJoinName") });
				REQUIRE_OP(Result);

				FOnlineSessionId SessionId = Result.GetOkValue().Session->GetSessionId();

				FindSessionsHelperParams.OpParams->SessionId = SessionId;
				JoinSessionHelperParams.OpParams->SessionId = SessionId;
			})
		.EmplaceStep<FFindSessionsHelper>(MoveTemp(FindSessionsHelperParams), ExpectedSessionsFound)
		.EmplaceStep<FJoinSessionHelper>(MoveTemp(JoinSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}