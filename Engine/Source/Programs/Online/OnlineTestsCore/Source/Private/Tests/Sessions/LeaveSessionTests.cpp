// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"

#define SESSIONS_TAG "[suite_sessions]"
#define SESSIONS_LEAVESESSION_TAG SESSIONS_TAG "[leavesesssion]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call LeaveSession with an invalid account id, I get an error", SESSIONS_LEAVESESSION_TAG)
{
	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FAccountId();
	LeaveSessionHelperParams.ExpectedError = TOnlineResult<FLeaveSession>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call LeaveSession with an empty session name, I get an error", SESSIONS_LEAVESESSION_TAG)
{
	FAccountId AccountId;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("");
	LeaveSessionHelperParams.ExpectedError = TOnlineResult<FLeaveSession>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call LeaveSession with an unregistered session name, I get an error", SESSIONS_LEAVESESSION_TAG)
{
	FAccountId AccountId;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("LeaveUnregisteredSessionName");
	LeaveSessionHelperParams.ExpectedError = TOnlineResult<FLeaveSession>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call LeaveSession with valid data, the operation completes successfully", SESSIONS_LEAVESESSION_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("LeaveSessionNameValidLeave");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("LeaveSessionNameValidLeave");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}
