// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Online/OnlineServicesLog.h"
#include "EOSShared.h"
#include "Helpers/TickForTime.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_PRESENCESESSION_TAG SESSIONS_TAG "[presencesession]"
#define EG_SESSIONS_PRESENCESESSIONEOS_TAG SESSIONS_TAG "[presencesession][.EOS]"
#define EG_SESSIONS_PRESENCESESSIONNULL_TAG SESSIONS_TAG "[presencesession][.NULL]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call IsPresenceSession with an invalid account id, I get an error", EG_SESSIONS_PRESENCESESSION_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FIsPresenceSession::Params OpIsPresenceParams;
				OpIsPresenceParams.LocalAccountId = FAccountId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FIsPresenceSession> Result = SessionsInterface->IsPresenceSession(MoveTemp(OpIsPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call IsPresenceSession with an invalid session id, I get an error", EG_SESSIONS_PRESENCESESSION_TAG)
{
	FAccountId AccountId;

	GetLoginPipeline({ AccountId })
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FIsPresenceSession::Params OpIsPresenceParams;
				OpIsPresenceParams.LocalAccountId = AccountId;
				OpIsPresenceParams.SessionId = FOnlineSessionId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FIsPresenceSession> Result = SessionsInterface->IsPresenceSession(MoveTemp(OpIsPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call IsPresenceSession with an unregistered account id, I get an error", EG_SESSIONS_PRESENCESESSION_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("IsPresenceUnregisteredSessionName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("IsPresenceUnregisteredSessionName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FIsPresenceSession::Params OpIsPresenceParams;

	FGetSessionByName::Params OpGetByNameParams;
	OpGetByNameParams.LocalName = TEXT("IsPresenceUnregisteredSessionName");

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;
	OpIsPresenceParams.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&OpIsPresenceParams, &OpGetByNameParams](const IOnlineServicesPtr& OnlineSubsystem)
			{			
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName(MoveTemp(OpGetByNameParams));
				REQUIRE_OP(Result);

				OpIsPresenceParams.SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams))
		.EmplaceLambda([&OpIsPresenceParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FIsPresenceSession> Result = SessionsInterface->IsPresenceSession(MoveTemp(OpIsPresenceParams));

				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidState());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call IsPresenceSession with valid information, it returns true if the session matches, and error if it does not", EG_SESSIONS_PRESENCESESSIONEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpFirstCreateParams;
	FCreateSessionHelper::FHelperParams FirstCreateSessionHelperParams;
	FirstCreateSessionHelperParams.OpParams = &OpFirstCreateParams;
	FirstCreateSessionHelperParams.OpParams->SessionName = TEXT("IsPresenceEnableSessionName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	FirstCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	FirstCreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FCreateSession::Params OpSecondCreateParams;
	FCreateSessionHelper::FHelperParams SecondCreateSessionHelperParams;
	SecondCreateSessionHelperParams.OpParams = &OpSecondCreateParams;
	SecondCreateSessionHelperParams.OpParams->SessionName = TEXT("IsPresenceDisableSessionName");
	SecondCreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	SecondCreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName1");
	SecondCreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FIsPresenceSession::Params OpFirstIsPresenceParams;
	FIsPresenceSession::Params OpSecondIsPresenceParams;

	FLeaveSession::Params OpFirstLeaveParams;
	FLeaveSessionHelper::FHelperParams FirstLeaveSessionHelperParams;
	FirstLeaveSessionHelperParams.OpParams = &OpFirstLeaveParams;
	FirstLeaveSessionHelperParams.OpParams->SessionName = TEXT("IsPresenceEnableSessionName");
	FirstLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FLeaveSession::Params OpSecondLeaveParams;
	FLeaveSessionHelper::FHelperParams SecondLeaveSessionHelperParams;
	SecondLeaveSessionHelperParams.OpParams = &OpSecondLeaveParams;
	SecondLeaveSessionHelperParams.OpParams->SessionName = TEXT("IsPresenceDisableSessionName");
	SecondLeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });
	 
	FirstCreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondCreateSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;
	
	FirstLeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SecondLeaveSessionHelperParams.OpParams->LocalAccountId = SecondAccountId;

	OpFirstIsPresenceParams.LocalAccountId = FirstAccountId;
	OpSecondIsPresenceParams.LocalAccountId = SecondAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(FirstCreateSessionHelperParams))
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(SecondCreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(6000))
		.EmplaceLambda([&OpFirstIsPresenceParams, &OpSecondIsPresenceParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("IsPresenceEnableSessionName") });
				REQUIRE_OP(Result);

				OpFirstIsPresenceParams.SessionId = Result.GetOkValue().Session->GetSessionId();

				Result = SessionsInterface->GetSessionByName({ TEXT("IsPresenceDisableSessionName") });
				REQUIRE_OP(Result);

				OpSecondIsPresenceParams.SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceLambda([&OpFirstIsPresenceParams, &OpSecondIsPresenceParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FIsPresenceSession> Result = SessionsInterface->IsPresenceSession(MoveTemp(OpFirstIsPresenceParams));
				REQUIRE_OP(Result);
				CHECK(Result.GetOkValue().bIsPresenceSession == true);

				Result = SessionsInterface->IsPresenceSession(MoveTemp(OpSecondIsPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidState());
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(FirstLeaveSessionHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(SecondLeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SetPresenceSession for EOS, I get an error", EG_SESSIONS_PRESENCESESSIONEOS_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FSetPresenceSession::Params OpSetPresenceParams;
				
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FSetPresenceSession> Result = SessionsInterface->SetPresenceSession(MoveTemp(OpSetPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::NotImplemented());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SetPresenceSession with an invalid account id, I get an error", EG_SESSIONS_PRESENCESESSIONNULL_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FSetPresenceSession::Params OpSetPresenceParams;
				OpSetPresenceParams.LocalAccountId = FAccountId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FSetPresenceSession> Result = SessionsInterface->SetPresenceSession(MoveTemp(OpSetPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SetPresenceSession with an invalid session id, I get an error", EG_SESSIONS_PRESENCESESSIONNULL_TAG)
{
	FAccountId AccountId;
	GetLoginPipeline({ AccountId })
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FSetPresenceSession::Params OpSetPresenceParams;
				OpSetPresenceParams.LocalAccountId = AccountId;
				OpSetPresenceParams.SessionId = FOnlineSessionId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FSetPresenceSession> Result = SessionsInterface->SetPresenceSession(MoveTemp(OpSetPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SetPresenceSession with valid data, the operation completes successfully", EG_SESSIONS_PRESENCESESSIONNULL_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SetPresenceValidName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = false;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SetPresenceValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FSetPresenceSession::Params OpSetPresenceParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;
	OpSetPresenceParams.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&OpSetPresenceParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName({ TEXT("SetPresenceValidName") });
				REQUIRE_OP(Result);

				OpSetPresenceParams.SessionId = Result.GetOkValue().Session->GetSessionId();
			})
		.EmplaceLambda([&OpSetPresenceParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FSetPresenceSession> Result = SessionsInterface->SetPresenceSession(MoveTemp(OpSetPresenceParams));
				REQUIRE_OP(Result);
				
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call ClearPresenceSession for EOS, I get an error", EG_SESSIONS_PRESENCESESSIONEOS_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FClearPresenceSession::Params OpClearPresenceParams;

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FClearPresenceSession> Result = SessionsInterface->ClearPresenceSession(MoveTemp(OpClearPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::NotImplemented());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call ClearPresenceSession with an invalid account id, I get an error", EG_SESSIONS_PRESENCESESSIONNULL_TAG)
{
	GetPipeline()
		.EmplaceLambda([](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FClearPresenceSession::Params OpClearPresenceParams;
				OpClearPresenceParams.LocalAccountId = FAccountId();

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FClearPresenceSession> Result = SessionsInterface->ClearPresenceSession(MoveTemp(OpClearPresenceParams));
				REQUIRE(Result.IsError());
				CHECK(Result.GetErrorValue() == Errors::InvalidParams());
			});

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call ClearPresenceSession with valid data, the operation completes successfully", EG_SESSIONS_PRESENCESESSIONNULL_TAG)
{
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("ClearPresenceValidName");
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("ClearPresenceValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	
	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceLambda([&AccountId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				FClearPresenceSession::Params OpClearPresenceParams;
				OpClearPresenceParams.LocalAccountId = AccountId;

				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FClearPresenceSession> Result = SessionsInterface->ClearPresenceSession(MoveTemp(OpClearPresenceParams));
				REQUIRE_OP(Result);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}