// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/SendRejectSessionInviteHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "EOSShared.h"
#include "Helpers/TickForTime.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_SENDREJECTSESSIONINVITE_TAG SESSIONS_TAG "[sendrejectsesssioninvite]"
#define EG_SESSIONS_SENDREJECTSESSIONINVITEEOS_TAG SESSIONS_TAG "[sendrejectsesssioninvite][.EOS]"
#define EG_SESSIONS_SENDREJECTSESSIONINVITE_MULTIACCOUNT_TAG SESSIONS_TAG "[sendrejectsesssioninvite][MultiAccount]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call SendSessionInvite with an invalid account id, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_TAG)
{
	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("SessionSendInviteInvalidIdName");
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FAccountId();
	SendSessionInviteHelperParams.ExpectedError = TOnlineResult<FSendSessionInvite>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SendSessionInvite with an empty session name, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_TAG)
{
	FAccountId AccountId;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("");
	SendSessionInviteHelperParams.ExpectedError = TOnlineResult<FSendSessionInvite>(Errors::InvalidParams());
	
	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	SendSessionInviteHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SendSessionInvite with a valid session name but unregistered, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_MULTIACCOUNT_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("SessionSendInviteUnregisteredName");
	SendSessionInviteHelperParams.ExpectedError = TOnlineResult<FSendSessionInvite>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(SecondAccountId);

	LoginPipeline
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SendSessionInvite with an empty target users, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_MULTIACCOUNT_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionSendInviteEmptyName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("SessionSendInviteEmptyName");
	SendSessionInviteHelperParams.ExpectedError = TOnlineResult<FSendSessionInvite>(Errors::InvalidParams());
	
	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionSendInviteEmptyName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SendSessionInvite with an invalid target users, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_MULTIACCOUNT_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;
		
	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionSendInviteInvalidUsersName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("SendSessionInviteName");
	SendSessionInviteHelperParams.ExpectedError = TOnlineResult<FSendSessionInvite>(Errors::InvalidParams());
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(FAccountId());

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionSendInviteInvalidUsersName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call SendSessionInvite with valid data, the operation completes successfully", EG_SESSIONS_SENDREJECTSESSIONINVITEEOS_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionSendInviteValidName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("SessionSendInviteValidName");

	FGetAllSessionInvites::Params OpGetAllSessionInvitesParams;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionSendInviteValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	OpGetAllSessionInvitesParams.LocalAccountId = SecondAccountId;
	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(SecondAccountId);
	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionInvitesNum = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceLambda([&OpGetAllSessionInvitesParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetAllSessionInvites> Result = SessionsInterface->GetAllSessionInvites(MoveTemp(OpGetAllSessionInvitesParams));
				CHECK(Result.GetOkValue().SessionInvites.IsEmpty());
			})
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceLambda([&OpGetAllSessionInvitesParams, &ExpectedSessionInvitesNum](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetAllSessionInvites> Result = SessionsInterface->GetAllSessionInvites(MoveTemp(OpGetAllSessionInvitesParams));
				CHECK(Result.GetOkValue().SessionInvites.Num() == ExpectedSessionInvitesNum);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RejectSessionInvite with an invalid account id, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_TAG)
{
	FRejectSessionInvite::Params OpRejectInviteParams;
	FRejectSessionInviteHelper::FHelperParams RejectSessionInviteHelperParams;
	RejectSessionInviteHelperParams.OpParams = &OpRejectInviteParams;
	RejectSessionInviteHelperParams.OpParams->LocalAccountId = FAccountId();
	RejectSessionInviteHelperParams.ExpectedError = TOnlineResult<FRejectSessionInvite>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FRejectSessionInviteHelper>(MoveTemp(RejectSessionInviteHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RejectSessionInvite with an invalid session invite id, I get an error", EG_SESSIONS_SENDREJECTSESSIONINVITE_TAG)
{
	FAccountId AccountId;

	FRejectSessionInvite::Params OpRejectInviteParams;
	FRejectSessionInviteHelper::FHelperParams RejectSessionInviteHelperParams;
	RejectSessionInviteHelperParams.OpParams = &OpRejectInviteParams;
	RejectSessionInviteHelperParams.OpParams->SessionInviteId = FSessionInviteId();
	RejectSessionInviteHelperParams.ExpectedError = TOnlineResult<FRejectSessionInvite>(Errors::InvalidParams());
	
	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });
	
	RejectSessionInviteHelperParams.OpParams->LocalAccountId = AccountId;
	
	LoginPipeline
		.EmplaceStep<FRejectSessionInviteHelper>(MoveTemp(RejectSessionInviteHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call RejectSessionInvite with valid data, the operation completes successfully", EG_SESSIONS_SENDREJECTSESSIONINVITEEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogEOSSDK, ELogVerbosity::NoLogging);

	int32 TestAccountIndex = 7;
	FAccountId FirstAccountId, SecondAccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("SessionRejectInviteValidName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FSendSessionInvite::Params OpSendInviteParams;
	FSendSessionInviteHelper::FHelperParams SendSessionInviteHelperParams;
	SendSessionInviteHelperParams.OpParams = &OpSendInviteParams;
	SendSessionInviteHelperParams.OpParams->SessionName = TEXT("SessionRejectInviteValidName");

	FRejectSessionInvite::Params OpRejectInviteParams;
	FRejectSessionInviteHelper::FHelperParams RejectSessionInviteHelperParams;
	RejectSessionInviteHelperParams.OpParams = &OpRejectInviteParams;

	FGetAllSessionInvites::Params OpGetAllSessionInvitesParams;

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("SessionRejectInviteValidName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	OpGetAllSessionInvitesParams.LocalAccountId = SecondAccountId;
	CreateSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;
	RejectSessionInviteHelperParams.OpParams->LocalAccountId = SecondAccountId;

	SendSessionInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendSessionInviteHelperParams.OpParams->TargetUsers.Add(SecondAccountId);

	LeaveSessionHelperParams.OpParams->LocalAccountId = FirstAccountId;

	constexpr uint32_t ExpectedSessionInvitesNum = 1;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceStep<FSendSessionInviteHelper>(MoveTemp(SendSessionInviteHelperParams), [&RejectSessionInviteHelperParams](const UE::Online::FSessionInviteId& InSessionInviteId) 
			{
				RejectSessionInviteHelperParams.OpParams->SessionInviteId = InSessionInviteId;
			})
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceLambda([&OpGetAllSessionInvitesParams, &ExpectedSessionInvitesNum](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetAllSessionInvites> Result = SessionsInterface->GetAllSessionInvites(MoveTemp(OpGetAllSessionInvitesParams));
				CHECK(Result.GetOkValue().SessionInvites.Num() == ExpectedSessionInvitesNum);
			})
		.EmplaceStep<FRejectSessionInviteHelper>(MoveTemp(RejectSessionInviteHelperParams))
		.EmplaceStep<FTickForTime>(FTimespan::FromMilliseconds(1000))
		.EmplaceLambda([&OpGetAllSessionInvitesParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetAllSessionInvites> Result = SessionsInterface->GetAllSessionInvites(MoveTemp(OpGetAllSessionInvitesParams));
				CHECK(Result.GetOkValue().SessionInvites.IsEmpty());
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));
			
	RunToCompletion();
}
