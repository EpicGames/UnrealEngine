// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/StartMatchmakingSessionHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Online/OnlineServicesLog.h"

#define SESSIONS_TAG "[suite_sessions]"
#define EG_SESSIONS_STARTMATCHMAKING_TAG SESSIONS_TAG "[startmatchmaking]"
#define EG_SESSIONS_STARTMATCHMAKINGEOS_TAG SESSIONS_TAG "[startmatchmaking][.EOS]"
#define EG_SESSIONS_STARTMATCHMAKING_SKIP_TAG SESSIONS_TAG "[startmatchmaking][startmatchmakingskip]"
#define SESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAG __VA_ARGS__)

SESSIONS_TEST_CASE("If I call StartMatchmaking with an invalid account id, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = FAccountId();
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));
		
	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with an empty session name, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	FAccountId AccountId;
 
	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("");
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with an empty schema name in settings, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	FAccountId AccountId;

	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchEmptySchemaSessionName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("");
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with an invalid max connections number in settings, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	FAccountId AccountId;

	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchInvalidNumConnSessionName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("InvalidNumConnSchemaName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 0;
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with an empty custom setting name in settings, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	FAccountId AccountId;

	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchEmptySettingSessionName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("EmptySettingSchemaName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 2;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.CustomSettings.Emplace(FName(""), FCustomSessionSetting{FSchemaVariant(false), ESchemaAttributeVisibility::Public});
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with an empty search filter key, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	FAccountId AccountId;

	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchEmptyFilterSessionName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("EmptyFilterSchemaName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 2;
	StartMatchmakingHelperParams.OpParams->SessionSearchFilters.Emplace(FFindSessionsSearchFilter{ FName(""), ESchemaAttributeComparisonOp::Equals, FSchemaVariant(100.0)});
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with session override id less than 16 chars or more than 64, I get an error", EG_SESSIONS_STARTMATCHMAKINGEOS_TAG)
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogOnlineServices, ELogVerbosity::NoLogging);

	FAccountId AccountId;

	FStartMatchmaking::Params OpFirstStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams FirstStartMatchmakingHelperParams;
	FirstStartMatchmakingHelperParams.OpParams = &OpFirstStartMatchmakingParams;
	FirstStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchmakingAlreadyInUseSessionName");
	FirstStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("SchemaName");
	FirstStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 2;
	FirstStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionIdOverride = TEXT("SessionId");
	FirstStartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FStartMatchmaking::Params OpSecondStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams SecondStartMatchmakingHelperParams;
	SecondStartMatchmakingHelperParams.OpParams = &OpSecondStartMatchmakingParams;
	SecondStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchmakingAlreadyInUseSessionName");
	SecondStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("SchemaName");
	SecondStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 2;
	SecondStartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionIdOverride = TEXT("SessionIdOverrideSessionIdOverrideSessionIdOverrideSessionIdOverride");
	SecondStartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	FirstStartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;
	SecondStartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(FirstStartMatchmakingHelperParams))
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(SecondStartMatchmakingHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with name already in use, I get an error", EG_SESSIONS_STARTMATCHMAKING_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("StartMatchmakingAlreadyInUseSessionName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 2;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;

	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchmakingAlreadyInUseSessionName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("SchemaName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 2;
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::InvalidState());

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("StartMatchmakingAlreadyInUseSessionName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams))
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}

SESSIONS_TEST_CASE("If I call StartMatchmaking with valid data, the operation will end with NotImplemented error for EOS and NULL", EG_SESSIONS_STARTMATCHMAKING_SKIP_TAG)
{
	FAccountId AccountId;

	FStartMatchmaking::Params OpStartMatchmakingParams;
	FStartMatchmakingHelper::FHelperParams StartMatchmakingHelperParams;
	StartMatchmakingHelperParams.OpParams = &OpStartMatchmakingParams;
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionName = TEXT("StartMatchEmptySettingSessionName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.SchemaName = TEXT("EmptySettingSchemaName");
	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.SessionSettings.NumMaxConnections = 2;
	StartMatchmakingHelperParams.ExpectedError = TOnlineResult<FStartMatchmaking>(Errors::NotImplemented());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	StartMatchmakingHelperParams.OpParams->SessionCreationParameters.LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FStartMatchmakingHelper>(MoveTemp(StartMatchmakingHelperParams));

	RunToCompletion();
}
