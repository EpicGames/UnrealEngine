// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Sessions/CreateSessionHelper.h"
#include "Helpers/Sessions/UpdateSessionSettingsHelper.h"
#include "Helpers/Sessions/LeaveSessionHelper.h"
#include "Algo/Count.h"

#define SESSIONS_TAGS "[suite_sessions]"
#define SESSIONS_UPDATESESSIONSETTINGS_TAG SESSIONS_TAGS "[updatesessionsettings]"
#define UPDATESESSIONS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SESSIONS_TAGS __VA_ARGS__)

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with an invalid account id, I get an error", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = FAccountId();
	UpdateSessionSettingsHelperParams.ExpectedError = TOnlineResult<FUpdateSessionSettings>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams));

	RunToCompletion();
}

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with an empty session name, I get an error", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	FAccountId AccountId;

	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("");
	UpdateSessionSettingsHelperParams.ExpectedError = TOnlineResult<FUpdateSessionSettings>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams));

	RunToCompletion();
}

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with an empty schema name in settings, I get an error", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	FAccountId AccountId;

	FUpdateSessionSettings::Params OpParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("EmptySchemaSessionName");
	UpdateSessionSettingsHelperParams.OpParams->Mutations.SchemaName = TEXT("");
	UpdateSessionSettingsHelperParams.ExpectedError = TOnlineResult<FUpdateSessionSettings>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams));

	RunToCompletion();
}

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with an invalid max connections number in settings, I get an error", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	FAccountId AccountId;

	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("InvalidMaxConnectionsSessionName");
	UpdateSessionSettingsHelperParams.OpParams->Mutations.NumMaxConnections = 0;
	UpdateSessionSettingsHelperParams.ExpectedError = TOnlineResult<FUpdateSessionSettings>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams));

	RunToCompletion();
}

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with an empty custom setting name in settings, I get an error", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	FAccountId AccountId;

	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("EmptyCustomSettingsSessionName");
	UpdateSessionSettingsHelperParams.OpParams->Mutations.UpdatedCustomSettings.Emplace(TEXT(""), FCustomSessionSetting{FSchemaVariant(false), ESchemaAttributeVisibility::Public});
	UpdateSessionSettingsHelperParams.ExpectedError = TOnlineResult<FUpdateSessionSettings>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams));

	RunToCompletion();
}

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with an unregistered session name, I get an error", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	FAccountId AccountId;

	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("UnregisteredUpdateSessionName");
	UpdateSessionSettingsHelperParams.ExpectedError = TOnlineResult<FUpdateSessionSettings>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams));

	RunToCompletion();
}

UPDATESESSIONS_TEST_CASE("If I call UpdateSessionSettings with valid data, the operation completes successfully", SESSIONS_UPDATESESSIONSETTINGS_TAG)
{
	int32 TestAccountIndex = 7;
	FAccountId AccountId;

	FCreateSession::Params OpCreateParams;
	FCreateSessionHelper::FHelperParams CreateSessionHelperParams;
	CreateSessionHelperParams.OpParams = &OpCreateParams;
	CreateSessionHelperParams.OpParams->SessionName = TEXT("ValidUpdateSessionName");
	CreateSessionHelperParams.OpParams->SessionSettings.SchemaName = TEXT("SchemaName");
	CreateSessionHelperParams.OpParams->SessionSettings.NumMaxConnections = 4;
	CreateSessionHelperParams.OpParams->bPresenceEnabled = true;
	CreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(TEXT("IntSetting"), FCustomSessionSetting{ FSchemaVariant(int64(50)), ESchemaAttributeVisibility::Public });
	CreateSessionHelperParams.OpParams->SessionSettings.CustomSettings.Emplace(TEXT("StringSetting"), FCustomSessionSetting{ FSchemaVariant(TEXT("TestString")), ESchemaAttributeVisibility::Public});

	FUpdateSessionSettings::Params OpUpdateParams;
	FUpdateSessionSettingsHelper::FHelperParams UpdateSessionSettingsHelperParams;
	UpdateSessionSettingsHelperParams.OpParams = &OpUpdateParams;
	UpdateSessionSettingsHelperParams.OpParams->SessionName = TEXT("ValidUpdateSessionName");
	UpdateSessionSettingsHelperParams.OpParams->Mutations.bAllowNewMembers = false;
	UpdateSessionSettingsHelperParams.OpParams->Mutations.JoinPolicy = ESessionJoinPolicy::InviteOnly;
	UpdateSessionSettingsHelperParams.OpParams->Mutations.NumMaxConnections = 8;
	UpdateSessionSettingsHelperParams.OpParams->Mutations.SchemaName = TEXT("ChangedSchemaName");
	UpdateSessionSettingsHelperParams.OpParams->Mutations.UpdatedCustomSettings.Emplace(TEXT("IntSetting"), FCustomSessionSetting{ FSchemaVariant(int64(100)), ESchemaAttributeVisibility::Public });
	UpdateSessionSettingsHelperParams.OpParams->Mutations.UpdatedCustomSettings.Emplace(TEXT("DoubleSetting"), FCustomSessionSetting{ FSchemaVariant(100.0), ESchemaAttributeVisibility::Public });
	UpdateSessionSettingsHelperParams.OpParams->Mutations.RemovedCustomSettings.Add(TEXT("StringSetting"));

	FGetSessionByName::Params OpGetByNameParams;
	OpGetByNameParams.LocalName = TEXT("ValidUpdateSessionName");

	FLeaveSession::Params OpLeaveParams;
	FLeaveSessionHelper::FHelperParams LeaveSessionHelperParams;
	LeaveSessionHelperParams.OpParams = &OpLeaveParams;
	LeaveSessionHelperParams.OpParams->SessionName = TEXT("ValidUpdateSessionName");
	LeaveSessionHelperParams.OpParams->bDestroySession = true;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	CreateSessionHelperParams.OpParams->LocalAccountId = AccountId;
	UpdateSessionSettingsHelperParams.OpParams->LocalAccountId = AccountId;
	LeaveSessionHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FCreateSessionHelper>(MoveTemp(CreateSessionHelperParams))
		.EmplaceStep<FUpdateSessionSettingsHelper>(MoveTemp(UpdateSessionSettingsHelperParams))
		.EmplaceLambda([&OpGetByNameParams, &UpdateSessionSettingsHelperParams](const IOnlineServicesPtr& OnlineSubsystem)
			{
				ISessionsPtr SessionsInterface = OnlineSubsystem->GetSessionsInterface();
				TOnlineResult<FGetSessionByName> Result = SessionsInterface->GetSessionByName(MoveTemp(OpGetByNameParams));
				
				REQUIRE_OP(Result);
				const FSessionSettings& SessionSettings = Result.GetOkValue().Session->GetSessionSettings();

				CHECK(SessionSettings.CustomSettings.Contains("IntSetting"));
				CHECK(SessionSettings.CustomSettings.Contains("DoubleSetting"));
				CHECK(!SessionSettings.CustomSettings.Contains("StringSetting"));

				CHECK(SessionSettings.CustomSettings["IntSetting"].Data.GetInt64() == 100);
				CHECK(SessionSettings.CustomSettings["DoubleSetting"].Data.GetDouble() == 100.0);

				CHECK(SessionSettings.SchemaName == UpdateSessionSettingsHelperParams.OpParams->Mutations.SchemaName);
				CHECK(SessionSettings.JoinPolicy == UpdateSessionSettingsHelperParams.OpParams->Mutations.JoinPolicy);
				CHECK(SessionSettings.bAllowNewMembers == UpdateSessionSettingsHelperParams.OpParams->Mutations.bAllowNewMembers);
				CHECK(SessionSettings.NumMaxConnections == UpdateSessionSettingsHelperParams.OpParams->Mutations.NumMaxConnections);
			})
		.EmplaceStep<FLeaveSessionHelper>(MoveTemp(LeaveSessionHelperParams));

	RunToCompletion();
}
