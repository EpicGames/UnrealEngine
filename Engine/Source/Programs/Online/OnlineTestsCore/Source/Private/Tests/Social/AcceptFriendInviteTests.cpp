// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/AcceptFriendInviteHelper.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Online/AuthCommon.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_ACCEPTFRIENDINVITE_TAG SOCIAL_TAG "[acceptfriendinvite]"
#define EG_SOCIAL_ACCEPTFRIENDINVITEEOS_TAG SOCIAL_TAG "[acceptfriendinvite][.EOS]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns an error if call with an invalid local user account id", EG_SOCIAL_ACCEPTFRIENDINVITE_TAG)
{
	FAcceptFriendInvite::Params OpAcceptFriendInviteParams;
	FAcceptFriendInviteHelper::FHelperParams AcceptInviteHelperParams;
	AcceptInviteHelperParams.OpParams = &OpAcceptFriendInviteParams;
	AcceptInviteHelperParams.OpParams->LocalAccountId = FAccountId();
	AcceptInviteHelperParams.ExpectedError = TOnlineResult<FAcceptFriendInvite>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FAcceptFriendInviteHelper>(MoveTemp(AcceptInviteHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns an error if call with an target user account id", EG_SOCIAL_ACCEPTFRIENDINVITE_TAG)
{
	FAccountId AccountId;

	FAcceptFriendInvite::Params OpAcceptFriendInviteParams;
	FAcceptFriendInviteHelper::FHelperParams AcceptInviteHelperParams;
	AcceptInviteHelperParams.OpParams = &OpAcceptFriendInviteParams;
	AcceptInviteHelperParams.OpParams->TargetAccountId = FAccountId();
	AcceptInviteHelperParams.ExpectedError = TOnlineResult<FAcceptFriendInvite>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	AcceptInviteHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FAcceptFriendInviteHelper>(MoveTemp(AcceptInviteHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns a fail message if the local user is not logged in", EG_SOCIAL_ACCEPTFRIENDINVITEEOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	int32 TestAccountIndex = 1;
	TSharedPtr<FPlatformUserId> FirstAccountPlatformUserId = MakeShared<FPlatformUserId>();
	TSharedPtr<FPlatformUserId> SecondAccountPlatformUserId = MakeShared<FPlatformUserId>();

	bool bLogout = false;

	FAcceptFriendInvite::Params OpAcceptFriendInviteParams;
	FAcceptFriendInviteHelper::FHelperParams AcceptInviteHelperParams;
	AcceptInviteHelperParams.OpParams = &OpAcceptFriendInviteParams;
	AcceptInviteHelperParams.ExpectedError = TOnlineResult<FAcceptFriendInvite>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	AcceptInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	AcceptInviteHelperParams.OpParams->TargetAccountId = SecondAccountId;

	LoginPipeline
		.EmplaceLambda([&FirstAccountId, &SecondAccountId, FirstAccountPlatformUserId, SecondAccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);

				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> FirstUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ FirstAccountId });
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> SecondUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ SecondAccountId });

				REQUIRE(FirstUserPlatformUserIdResult.IsOk());
				REQUIRE(SecondUserPlatformUserIdResult.IsOk());

				CHECK(FirstUserPlatformUserIdResult.TryGetOkValue() != nullptr);
				CHECK(SecondUserPlatformUserIdResult.TryGetOkValue() != nullptr);

				*FirstAccountPlatformUserId = FirstUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
				*SecondAccountPlatformUserId = SecondUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(FirstAccountPlatformUserId))
		.EmplaceStep<FAcceptFriendInviteHelper>(MoveTemp(AcceptInviteHelperParams))
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(SecondAccountPlatformUserId));

	RunToCompletion(bLogout);
}

//SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns a fail message if ERelationship with target user is Friend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns a fail message if ERelationship with target user is NotFriend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns a fail message if ERelationship with target user is InviteSent")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that AcceptFriendInvite successfully completes if ERelationship with target user is InviteReceived, ERelationship becomes Friend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that AcceptFriendInvite returns a fail message if ERelationship with target user is Blocked")
//{
//	// TODO
//}
