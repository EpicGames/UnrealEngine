// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/BlockUserHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_BLOCKUSER_TAG SOCIAL_TAG "[blockuser]"
#define EG_SOCIAL_BLOCKUSEREOS_TAG SOCIAL_TAG "[blockuser][.EOS]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that BlockUser returns a error if call with an invalid local user account id", EG_SOCIAL_BLOCKUSER_TAG)
{
	FBlockUser::Params OpBlockUserParams;
	FBlockUserHelper::FHelperParams BlockUserHelperHelperParams;
	BlockUserHelperHelperParams.OpParams = &OpBlockUserParams;
	BlockUserHelperHelperParams.OpParams->LocalAccountId = FAccountId();
	BlockUserHelperHelperParams.ExpectedError = TOnlineResult<FBlockUser>(Errors::InvalidParams());


	GetPipeline()
		.EmplaceStep<FBlockUserHelper>(MoveTemp(BlockUserHelperHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that BlockUser returns a error if call with an target user account id", EG_SOCIAL_BLOCKUSER_TAG)
{
	FAccountId AccountId;

	FBlockUser::Params OpBlockUserParams;
	FBlockUserHelper::FHelperParams BlockUserHelperHelperParams;
	BlockUserHelperHelperParams.OpParams = &OpBlockUserParams;
	BlockUserHelperHelperParams.OpParams->TargetAccountId = FAccountId();
	BlockUserHelperHelperParams.ExpectedError = TOnlineResult<FBlockUser>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	BlockUserHelperHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FBlockUserHelper>(MoveTemp(BlockUserHelperHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that BlockUser returns a fail message if the local user is not logged in", EG_SOCIAL_BLOCKUSEREOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;
	
	int32 TestAccountIndex = 1;
	TSharedPtr<FPlatformUserId> FirstAccountPlatformUserId = MakeShared<FPlatformUserId>();
	TSharedPtr<FPlatformUserId> SecondAccountPlatformUserId = MakeShared<FPlatformUserId>();

	bool bLogout = false;

	FBlockUser::Params OpBlockUserParams;
	FBlockUserHelper::FHelperParams BlockUserHelperHelperParams;
	BlockUserHelperHelperParams.OpParams = &OpBlockUserParams;
	BlockUserHelperHelperParams.ExpectedError = TOnlineResult<FBlockUser>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	BlockUserHelperHelperParams.OpParams->LocalAccountId = FirstAccountId;
	BlockUserHelperHelperParams.OpParams->TargetAccountId = SecondAccountId;

	LoginPipeline
		.EmplaceLambda([&FirstAccountId, &SecondAccountId, FirstAccountPlatformUserId, SecondAccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);

				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> FirstUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ FirstAccountId });
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> SecondUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ SecondAccountId });

				REQUIRE(FirstUserPlatformUserIdResult.IsOk());
				REQUIRE(SecondUserPlatformUserIdResult.IsOk());

				CHECK(FirstUserPlatformUserIdResult.TryGetOkValue() != nullptr);
				CHECK(SecondUserPlatformUserIdResult.TryGetOkValue() != nullptr);

				*FirstAccountPlatformUserId = FirstUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
				*SecondAccountPlatformUserId = SecondUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(FirstAccountPlatformUserId))
		.EmplaceStep<FBlockUserHelper>(MoveTemp(BlockUserHelperHelperParams))
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(SecondAccountPlatformUserId));

	RunToCompletion(bLogout);
}

SOCIAL_TEST_CASE("Verify that BlockUser completes successfully if both users are logged in", EG_SOCIAL_BLOCKUSEREOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	FBlockUser::Params OpBlockUserParams;
	FBlockUserHelper::FHelperParams BlockUserHelperHelperParams;
	BlockUserHelperHelperParams.OpParams = &OpBlockUserParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ FirstAccountId, SecondAccountId });

	BlockUserHelperHelperParams.OpParams->LocalAccountId = FirstAccountId;
	BlockUserHelperHelperParams.OpParams->TargetAccountId = SecondAccountId;

	LoginPipeline
		.EmplaceStep<FBlockUserHelper>(MoveTemp(BlockUserHelperHelperParams));

	RunToCompletion();
}
