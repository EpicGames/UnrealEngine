// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/GetBlockedUsersHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_GETBLOCKEDUSERSGDK_TAG SOCIAL_TAG "[getblockedusers][.GDK]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that GetBlockedUsers returns a error if call with an invalid local user account id", EG_SOCIAL_GETBLOCKEDUSERSGDK_TAG)
{
	FGetBlockedUsers::Params OpGetBlockedUsersParams;
	FGetBlockedUsersHelper::FHelperParams GetBlockedUsersHelperParams;
	GetBlockedUsersHelperParams.OpParams = &OpGetBlockedUsersParams;
	GetBlockedUsersHelperParams.OpParams->LocalAccountId = FAccountId();

	IOnlineServicesPtr OnlineServices = GetServices();
	EOnlineServices ServicesProvider = OnlineServices->GetServicesProvider();

	if (ServicesProvider == EOnlineServices::Epic)
	{
		GetBlockedUsersHelperParams.ExpectedError = TOnlineResult<FGetBlockedUsers>(Errors::NotImplemented());

	}
	else if (ServicesProvider == EOnlineServices::Xbox)
	{
		GetBlockedUsersHelperParams.ExpectedError = TOnlineResult<FGetBlockedUsers>(Errors::InvalidUser());
	}

	GetPipeline()
		.EmplaceStep<FGetBlockedUsersHelper>(MoveTemp(GetBlockedUsersHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that GetBlockedUsers returns a fail message if there are no avoid users for local user", EG_SOCIAL_GETBLOCKEDUSERSGDK_TAG)
{
	FAccountId AccountId;

	FGetBlockedUsers::Params OpGetBlockedUsersParams;
	FGetBlockedUsersHelper::FHelperParams GetBlockedUsersHelperParams;
	GetBlockedUsersHelperParams.OpParams = &OpGetBlockedUsersParams;

	IOnlineServicesPtr OnlineServices = GetServices();
	EOnlineServices ServicesProvider = OnlineServices->GetServicesProvider();

	if (ServicesProvider == EOnlineServices::Epic)
	{
		GetBlockedUsersHelperParams.ExpectedError = TOnlineResult<FGetBlockedUsers>(Errors::NotImplemented());

	}
	else if (ServicesProvider == EOnlineServices::Xbox)
	{
		GetBlockedUsersHelperParams.ExpectedError = TOnlineResult<FGetBlockedUsers>(Errors::InvalidUser());
	}

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	GetBlockedUsersHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FGetBlockedUsersHelper>(MoveTemp(GetBlockedUsersHelperParams));

	RunToCompletion();
}

//SOCIAL_TEST_CASE("Verify that GetBlockedUsers returns an empty list if there are no cached blocked users")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that GetBlockedUsers returns a list of 1 blocked user if there is 1 cached bocked user")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that GetBlockedUsers returns a list of all cached blocked user if there are multiple cached blocked users")
//{
//	// TODO
//}
