// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/QueryFriendsHelper.h"
#include "Helpers/Social/GetFriendsHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_GETFRIENDSEOS_TAG SOCIAL_TAG "[getfriends][.EOS]"
#define EG_SOCIAL_GETFRIENDS_TAG SOCIAL_TAG "[getfriends]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that GetFriends returns an empty list if there are no cached Friends", EG_SOCIAL_GETFRIENDSEOS_TAG)
{
	FAccountId AccountId;	

	int32 TestAccountIndex = 5;

	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;
	GetFriendsHelperParams.ExpectedError = TOnlineResult<FGetFriends>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that GetFriends returns a list of 1 Friend if there is 1 cached Friend", EG_SOCIAL_GETFRIENDSEOS_TAG)
{
	FAccountId AccountId;
	
	int32 TestAccountIndex = 6;

	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;

	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	QueryFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;

	int32 ExpectedFriendsNum = 1;

	LoginPipeline
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams))
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams), ExpectedFriendsNum);

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that GetFriends returns a list of all cached Friends if there are multiple cached Friends", EG_SOCIAL_GETFRIENDSEOS_TAG)
{
	FAccountId AccountId;

	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;

	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	QueryFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;

	int32 ExpectedFriendsNum = 5;

	LoginPipeline
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams))
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams), ExpectedFriendsNum);

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that GetFriend returns a fail message if there are no cached Friends", EG_SOCIAL_GETFRIENDSEOS_TAG)
{
	FAccountId AccountId;
	
	int32 TestAccountIndex = 5;

	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;
	GetFriendsHelperParams.ExpectedError = TOnlineResult<FGetFriends>(Errors::InvalidState());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams));

	RunToCompletion();
}
