// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/QueryBlockedUsersHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_QUERYBLOCKEDUSERSGDK_TAG SOCIAL_TAG "[queryblockedusers][.GDK]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that QueryBlockedUsers returns a error if call with an invalid local user account id", EG_SOCIAL_QUERYBLOCKEDUSERSGDK_TAG)
{
	FQueryBlockedUsers::Params OpQueryParams;
	FQueryBlockedUsersHelper::FHelperParams QueryBlockedUsersParams;
	QueryBlockedUsersParams.OpParams = &OpQueryParams;
	QueryBlockedUsersParams.OpParams->LocalAccountId = FAccountId();
	
	IOnlineServicesPtr OnlineServices = GetServices();
	EOnlineServices ServicesProvider = OnlineServices->GetServicesProvider();
	
	if (ServicesProvider == EOnlineServices::Epic)
	{
		QueryBlockedUsersParams.ExpectedError = TOnlineResult<FQueryBlockedUsers>(Errors::NotImplemented());
	
	}
	else if (ServicesProvider == EOnlineServices::Xbox)
	{
		QueryBlockedUsersParams.ExpectedError = TOnlineResult<FQueryBlockedUsers>(Errors::InvalidUser());
	}
	
	GetPipeline()
		.EmplaceStep<FQueryBlockedUsersHelper>(MoveTemp(QueryBlockedUsersParams));
	
	RunToCompletion();
}

//SOCIAL_TEST_CASE("Verify that QueryBlockedUsers returns a fail message if the local user is not logged in")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that QueryBlockedUsers caches no blocked users if no blocked users exist for this user")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that QueryBlockedUsers caches one blocked user if only one blocked user exists for this user")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that QueryBlockedUsers caches all blocked users if multiple blocked users exist for this user")
//{
//	// TODO
//}
