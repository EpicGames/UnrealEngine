// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/QueryFriendsHelper.h"
#include "Helpers/Social/GetFriendsHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_QUERYFRIENDS_TAG SOCIAL_TAG "[queryfriends]"
#define EG_SOCIAL_QUERYFRIENDSEOS_TAG SOCIAL_TAG "[queryfriends][.EOS]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that QueryFriends returns a fail message if the local user is not logged in", EG_SOCIAL_QUERYFRIENDSEOS_TAG)
{
	FAccountId AccountId;

	int32 TestAccountIndex = 1;
	TSharedPtr<FPlatformUserId> AccountPlatformUserId = MakeShared<FPlatformUserId>();
	bool bLogout = false;

	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;
	QueryFriendsHelperParams.ExpectedError = TOnlineResult<FQueryFriends>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	QueryFriendsHelperParams.OpParams->LocalAccountId = AccountId;


	LoginPipeline
		.EmplaceLambda([&AccountId, AccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);

				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> UserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ AccountId });
				
				REQUIRE(UserPlatformUserIdResult.IsOk());
				CHECK(UserPlatformUserIdResult.TryGetOkValue() != nullptr);

				*AccountPlatformUserId = UserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(AccountPlatformUserId))
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams));

	RunToCompletion(bLogout);
}

SOCIAL_TEST_CASE("Verify that QueryFriends returns a error if call with an invalid account id", EG_SOCIAL_QUERYFRIENDS_TAG)
{
	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;
	QueryFriendsHelperParams.OpParams->LocalAccountId = FAccountId();
	QueryFriendsHelperParams.ExpectedError = TOnlineResult<FQueryFriends>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that QueryFriends caches no Friends if no Friends exist for this user", EG_SOCIAL_QUERYFRIENDSEOS_TAG)
{
	FAccountId AccountId;
	int32 TestAccountIndex = 5;

	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;
	
	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });

	QueryFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	
	LoginPipeline
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams))
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that QueryFriends caches one Friend if only one Friend exists for this user", EG_SOCIAL_QUERYFRIENDSEOS_TAG)
{
	FAccountId AccountId;
	int32 TestAccountIndex = 6;

	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;
		
	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;
		
	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { AccountId });
	
	QueryFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	
	int32 ExpectedFriendsNum = 1;

	LoginPipeline
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams))
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams), ExpectedFriendsNum);
	
	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that QueryFriends caches all Friends if multiple Friends exist for this user", EG_SOCIAL_QUERYFRIENDSEOS_TAG)
{
	FAccountId AccountId;
	int32 ExpectedFriendsNum = 5;

	FQueryFriends::Params OpQueryParams;
	FQueryFriendsHelper::FHelperParams QueryFriendsHelperParams;
	QueryFriendsHelperParams.OpParams = &OpQueryParams;

	FGetFriends::Params OpGetParams;
	FGetFriendsHelper::FHelperParams GetFriendsHelperParams;
	GetFriendsHelperParams.OpParams = &OpGetParams;

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	QueryFriendsHelperParams.OpParams->LocalAccountId = AccountId;
	GetFriendsHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FQueryFriendsHelper>(MoveTemp(QueryFriendsHelperParams))
		.EmplaceStep<FGetFriendsHelper>(MoveTemp(GetFriendsHelperParams), ExpectedFriendsNum);

	RunToCompletion();
}
