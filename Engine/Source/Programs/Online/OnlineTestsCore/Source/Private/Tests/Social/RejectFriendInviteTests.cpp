// Copyright Epic Games, Inc. All Rights Reserved.
	
#include "Helpers/Social/RejectFriendInviteHelper.h"
#include "Helpers/Auth/AuthLogout.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_REJECTFRIENDINVITE_TAG SOCIAL_TAG "[rejectfriendinvite]"
#define EG_SOCIAL_REJECTFRIENDINVITEEOS_TAG SOCIAL_TAG "[queryfriends][.EOS]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a error if call with an invalid local user account id", EG_SOCIAL_REJECTFRIENDINVITE_TAG)
{
	FRejectFriendInvite::Params OpRejectFriendInviteParams;
	FRejectFriendInviteHelper::FHelperParams RejectFriendInviteHelperParams;
	RejectFriendInviteHelperParams.OpParams = &OpRejectFriendInviteParams;
	RejectFriendInviteHelperParams.OpParams->LocalAccountId = FAccountId();
	RejectFriendInviteHelperParams.ExpectedError = TOnlineResult<FRejectFriendInvite>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FRejectFriendInviteHelper>(MoveTemp(RejectFriendInviteHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a error if call with an target user account id", EG_SOCIAL_REJECTFRIENDINVITE_TAG)
{
	FAccountId AccountId;

	FRejectFriendInvite::Params OpRejectFriendInviteParams;
	FRejectFriendInviteHelper::FHelperParams RejectFriendInviteHelperParams;
	RejectFriendInviteHelperParams.OpParams = &OpRejectFriendInviteParams;
	RejectFriendInviteHelperParams.OpParams->TargetAccountId = FAccountId();
	RejectFriendInviteHelperParams.ExpectedError = TOnlineResult<FRejectFriendInvite>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	RejectFriendInviteHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FRejectFriendInviteHelper>(MoveTemp(RejectFriendInviteHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a fail message if the local user is not logged in", EG_SOCIAL_REJECTFRIENDINVITEEOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	int32 TestAccountIndex = 1;
	TSharedPtr<FPlatformUserId> FirstAccountPlatformUserId = MakeShared<FPlatformUserId>();
	TSharedPtr<FPlatformUserId> SecondAccountPlatformUserId = MakeShared<FPlatformUserId>();
	bool bLogout = false;

	FRejectFriendInvite::Params OpRejectFriendInviteParams;
	FRejectFriendInviteHelper::FHelperParams RejectFriendInviteHelperParams;
	RejectFriendInviteHelperParams.OpParams = &OpRejectFriendInviteParams;
	RejectFriendInviteHelperParams.ExpectedError = TOnlineResult<FRejectFriendInvite>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	RejectFriendInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	RejectFriendInviteHelperParams.OpParams->TargetAccountId = SecondAccountId;

	LoginPipeline
		.EmplaceLambda([&FirstAccountId, &SecondAccountId, FirstAccountPlatformUserId, SecondAccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);

				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> FirstUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ FirstAccountId });
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> SecondUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ SecondAccountId });

				REQUIRE(FirstUserPlatformUserIdResult.IsOk());
				REQUIRE(SecondUserPlatformUserIdResult.IsOk());

				CHECK(FirstUserPlatformUserIdResult.TryGetOkValue() != nullptr);
				CHECK(SecondUserPlatformUserIdResult.TryGetOkValue() != nullptr);

				*FirstAccountPlatformUserId = FirstUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
				*SecondAccountPlatformUserId = SecondUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(FirstAccountPlatformUserId))
		.EmplaceStep<FRejectFriendInviteHelper>(MoveTemp(RejectFriendInviteHelperParams))
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(SecondAccountPlatformUserId));

	RunToCompletion(bLogout);
}

//SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a fail message if ERelationship with target user is Friend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a fail message if ERelationship with target user is NotFriend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a fail message if ERelationship with target user is InviteSent")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that RejectFriendInvite successfully completes if ERelationship with target user is InviteReceived, ERelationship becomes NotFriend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that RejectFriendInvite returns a fail message if ERelationship with target user is Blocked")
//{
//	// TODO
//}
