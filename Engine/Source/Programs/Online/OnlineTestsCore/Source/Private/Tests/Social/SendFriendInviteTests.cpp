// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Social/GetFriendsHelper.h"	
#include "Helpers/Social/QueryFriendsHelper.h"
#include "Helpers/Social/SendFriendInviteHelper.h"
#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/Auth/AuthLogout.h"
#include "Helpers/Auth/AuthLogin.h"

#define SOCIAL_TAG "[suite_social]"
#define EG_SOCIAL_SENDFRIENDINVITE_TAG SOCIAL_TAG "[sendfriendinvite]"
#define EG_SOCIAL_SENDFRIENDINVITEEOS_TAG SOCIAL_TAG "[sendfriendinvite][.EOS]"

#define SOCIAL_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, SOCIAL_TAG __VA_ARGS__)

SOCIAL_TEST_CASE("Verify that SendFriendInvite returns a fail message if use invalid local user account id", EG_SOCIAL_SENDFRIENDINVITE_TAG)
{
	FSendFriendInvite::Params OpSendFriendInviteParams;
	FSendFriendInviteHelper::FHelperParams SendFriendInviteHelperParams;
	SendFriendInviteHelperParams.OpParams = &OpSendFriendInviteParams;
	SendFriendInviteHelperParams.OpParams->LocalAccountId = FAccountId();
	SendFriendInviteHelperParams.ExpectedError = TOnlineResult<FSendFriendInvite>(Errors::InvalidParams());

	GetPipeline()
		.EmplaceStep<FSendFriendInviteHelper>(MoveTemp(SendFriendInviteHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that SendFriendInvite returns a fail message if use invalid target user account id", EG_SOCIAL_SENDFRIENDINVITE_TAG)
{
	FAccountId AccountId;

	FSendFriendInvite::Params OpSendFriendInviteParams;
	FSendFriendInviteHelper::FHelperParams SendFriendInviteHelperParams;
	SendFriendInviteHelperParams.OpParams = &OpSendFriendInviteParams;
	SendFriendInviteHelperParams.OpParams->TargetAccountId = FAccountId();
	SendFriendInviteHelperParams.ExpectedError = TOnlineResult<FSendFriendInvite>(Errors::InvalidParams());

	FTestPipeline& LoginPipeline = GetLoginPipeline({ AccountId });

	SendFriendInviteHelperParams.OpParams->LocalAccountId = AccountId;

	LoginPipeline
		.EmplaceStep<FSendFriendInviteHelper>(MoveTemp(SendFriendInviteHelperParams));

	RunToCompletion();
}

SOCIAL_TEST_CASE("Verify that SendFriendInvite returns a fail message if the local user is not logged in", EG_SOCIAL_SENDFRIENDINVITEEOS_TAG)
{
	FAccountId FirstAccountId, SecondAccountId;

	int32 TestAccountIndex = 1;
	TSharedPtr<FPlatformUserId> FirstAccountPlatformUserId = MakeShared<FPlatformUserId>();
	TSharedPtr<FPlatformUserId> SecondAccountPlatformUserId = MakeShared<FPlatformUserId>();
	bool bLogout = false;

	FSendFriendInvite::Params OpSendFriendInviteParams;
	FSendFriendInviteHelper::FHelperParams SendFriendInviteHelperParams;
	SendFriendInviteHelperParams.OpParams = &OpSendFriendInviteParams;
	SendFriendInviteHelperParams.ExpectedError = TOnlineResult<FSendFriendInvite>(Errors::NotLoggedIn());

	FTestPipeline& LoginPipeline = GetLoginPipeline(TestAccountIndex, { FirstAccountId, SecondAccountId });

	SendFriendInviteHelperParams.OpParams->LocalAccountId = FirstAccountId;
	SendFriendInviteHelperParams.OpParams->TargetAccountId = SecondAccountId;

	LoginPipeline
		.EmplaceLambda([&FirstAccountId, &SecondAccountId, FirstAccountPlatformUserId, SecondAccountPlatformUserId](const IOnlineServicesPtr& OnlineSubsystem)
			{
				UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();
				REQUIRE(OnlineAuthPtr);

				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> FirstUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ FirstAccountId });
				UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> SecondUserPlatformUserIdResult = OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({ SecondAccountId });

				REQUIRE(FirstUserPlatformUserIdResult.IsOk());
				REQUIRE(SecondUserPlatformUserIdResult.IsOk());

				CHECK(FirstUserPlatformUserIdResult.TryGetOkValue() != nullptr);
				CHECK(SecondUserPlatformUserIdResult.TryGetOkValue() != nullptr);

				*FirstAccountPlatformUserId = FirstUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
				*SecondAccountPlatformUserId = SecondUserPlatformUserIdResult.TryGetOkValue()->AccountInfo->PlatformUserId;
			})
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(FirstAccountPlatformUserId))
		.EmplaceStep<FSendFriendInviteHelper>(MoveTemp(SendFriendInviteHelperParams))
		.EmplaceStep<FAuthLogoutStep>(MoveTemp(SecondAccountPlatformUserId));

	RunToCompletion(bLogout);
}

//SOCIAL_TEST_CASE("Verify that SendFriendInvite returns fail message if ERelationship with target user is Friend")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that SendFriendInvite completes successfully if ERelationship with target user is NotFriend, ERelationship becomes InviteSent")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that SendFriendInvite completes successfully if ERelationship with target user is InviteSent, ERelationship remains InviteSent")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that SendFriendInvite completes successfully if ERelationship with target user is InviteReceived, ERelationship becomes InviteSent")
//{
//	// TODO
//}

//SOCIAL_TEST_CASE("Verify that SendFriendInvite returns fail message if ERelationship with target user is Blocked")
//{
//	// TODO
//}
