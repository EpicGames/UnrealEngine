// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/Identity/IdentityGetLoginByUserId.h"
#include "Helpers/Stats/UpdateStatsHelper.h"
#include "Online/Stats.h"
#include "Online/StatsCommon.h"

#define STATS_TAGS "[Stats]"
#define STATS_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, STATS_TAGS __VA_ARGS__)

FTestPipeline&& StatsFixture_1User_With_InitialRandomStats(const OnlineTestBase& InOnlineTestBase, FAccountId& AccountId)
{
	return InOnlineTestBase.GetLoginPipeline({ AccountId })
		.EmplaceAsyncLambda([&AccountId](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			int64 InitValue = (int64)FMath::RandRange(1, 100);
			TArray<FUserStats> UpdateUsersStats{
				{ AccountId, { { "Stat_Use_Largest", FStatValue(InitValue)}, {"Stat_Use_Smallest", FStatValue(InitValue)}, { "Stat_Use_Sum", FStatValue(InitValue)}, { "Stat_Use_Set", FStatValue(InitValue)}}},
			};
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountId, UpdateUsersStats);
		});
}

void QueryStats_Fixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FAccountId& LocalAccountId, const FAccountId& TargetAccountId, const TArray<FString>& StatNames, TMap<FString, FStatValue>& OutStats)
{
	Services->GetStatsInterface()->QueryStats({ LocalAccountId, TargetAccountId, StatNames }).OnComplete([Promise=MoveTemp(Promise), &OutStats](const TOnlineResult<FQueryStats>& Result) mutable
	{
		CHECK_OP(Result);
		if (Result.IsOk())
		{
			OutStats = Result.GetOkValue().Stats;
		}
		Promise->SetValue(true);
	});
}

STATS_TEST_CASE("UpdateStats modify method work as expected", "[tmpExcludedStats]")
{
	FAccountId AccountId;
	TMap<FString, FStatValue> InitialStats;
	TMap<FString, FStatValue> UpdatedStats;
	int64 Expected_Stat_Use_Set = (int64)FMath::RandRange(1, 100);

	StatsFixture_1User_With_InitialRandomStats(*this, AccountId)
		.EmplaceAsyncLambda([&AccountId, &InitialStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Use_Largest", "Stat_Use_Smallest", "Stat_Use_Sum", "Stat_Use_Set" }, InitialStats); 
		})
		.EmplaceAsyncLambda([&AccountId, &InitialStats, &Expected_Stat_Use_Set](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			TArray<FUserStats> UpdateUsersStats{ { AccountId, {
				{"Stat_Use_Largest", FStatValue(InitialStats["Stat_Use_Largest"].GetInt64() - 1)},
				{"Stat_Use_Smallest", FStatValue(InitialStats["Stat_Use_Largest"].GetInt64() + 1)},
				{"Stat_Use_Sum", FStatValue((int64)3)},
				{"Stat_Use_Set", FStatValue(Expected_Stat_Use_Set)}
			}}, };
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountId, UpdateUsersStats);
		})
		.EmplaceAsyncLambda([&AccountId, &UpdatedStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Use_Largest", "Stat_Use_Smallest", "Stat_Use_Sum", "Stat_Use_Set" }, UpdatedStats);
		})
		.EmplaceLambda([&AccountId, &InitialStats, &UpdatedStats, &Expected_Stat_Use_Set](const IOnlineServicesPtr& Services) {
			CHECK_EQUAL(UpdatedStats["Stat_Use_Largest"].GetInt64(), InitialStats["Stat_Use_Largest"].GetInt64()); // Setting to smaller value didn't take effect
			CHECK_EQUAL(UpdatedStats["Stat_Use_Smallest"].GetInt64(), InitialStats["Stat_Use_Smallest"].GetInt64());  // Setting to bigger value didn't take effect
			CHECK_EQUAL(UpdatedStats["Stat_Use_Sum"].GetInt64(), InitialStats["Stat_Use_Sum"].GetInt64() + 3);
			CHECK_EQUAL(UpdatedStats["Stat_Use_Set"].GetInt64(), Expected_Stat_Use_Set);
		})
		.EmplaceAsyncLambda([&AccountId, &InitialStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			TArray<FUserStats> UpdateUsersStats{ { AccountId, {
				{"Stat_Use_Largest", FStatValue(InitialStats["Stat_Use_Largest"].GetInt64() + 1)},
				{"Stat_Use_Smallest", FStatValue(InitialStats["Stat_Use_Smallest"].GetInt64() - 1)},
			}}, };
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountId, UpdateUsersStats);
		})
		.EmplaceAsyncLambda([&AccountId, &UpdatedStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Use_Largest", "Stat_Use_Smallest" }, UpdatedStats);
		})
		.EmplaceLambda([&InitialStats, &UpdatedStats](const IOnlineServicesPtr& Services) {
			CHECK_EQUAL(UpdatedStats["Stat_Use_Largest"].GetInt64(), InitialStats["Stat_Use_Largest"].GetInt64() + 1); // Setting to bigger value took effect
			CHECK_EQUAL(UpdatedStats["Stat_Use_Smallest"].GetInt64(), InitialStats["Stat_Use_Smallest"].GetInt64() - 1);  // Setting to smaller value took effect
		});

	RunToCompletion();
}

STATS_TEST_CASE("UpdateStats modify different types work as expected", "[tmpExcludedStats]")
{
	FAccountId AccountId;
	TMap<FString, FStatValue> InitialStats;
	TMap<FString, FStatValue> UpdatedStats;

	GetLoginPipeline({ AccountId })
		.EmplaceAsyncLambda([&AccountId](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountId, { { AccountId, { { "Stat_Type_Bool", FStatValue(true)}, {"Stat_Type_Double", FStatValue(9999.999)}}}, });
		})
		.EmplaceAsyncLambda([&AccountId, &InitialStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Type_Bool", "Stat_Type_Double" }, InitialStats); 
		})
		.EmplaceAsyncLambda([&AccountId, &InitialStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			TArray<FUserStats> UpdateUsersStats{ { AccountId, {
				{"Stat_Type_Bool", FStatValue(false)},
				{"Stat_Type_Double", FStatValue(InitialStats["Stat_Type_Double"].GetDouble() - 0.001)},
			}}, };
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountId, UpdateUsersStats);
		})
		.EmplaceAsyncLambda([&AccountId, &UpdatedStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Type_Bool", "Stat_Type_Double" }, UpdatedStats);
		})
		.EmplaceLambda([&InitialStats, &UpdatedStats](const IOnlineServicesPtr& Services) {
			CHECK_EQUAL(UpdatedStats["Stat_Type_Bool"].GetBoolean(), false);
			CHECK(FMath::IsNearlyEqual(UpdatedStats["Stat_Type_Double"].GetDouble(), InitialStats["Stat_Type_Double"].GetDouble() - 0.001, UE_DOUBLE_SMALL_NUMBER));
		});

	RunToCompletion();
}

STATS_TEST_CASE("ResetStats works or not implemented", "[tmpExcludedStats]")
{
	FAccountId AccountId;
	TMap<FString, FStatValue> UserStats;
	bool ImplementedStatsReset = false;

	StatsFixture_1User_With_InitialRandomStats(*this, AccountId)
		.EmplaceAsyncLambda([&AccountId, &UserStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Use_Largest", "Stat_Use_Smallest", "Stat_Use_Sum", "Stat_Use_Set" }, UserStats); 
		})
		.EmplaceLambda([&UserStats](const IOnlineServicesPtr& Services) {
			CHECK(UserStats.Find("Stat_Use_Largest") != nullptr);
			CHECK(UserStats.Find("Stat_Use_Smallest") != nullptr);
			CHECK(UserStats.Find("Stat_Use_Sum") != nullptr);
			CHECK(UserStats.Find("Stat_Use_Set") != nullptr);
		})
		.EmplaceAsyncLambda([&AccountId, &ImplementedStatsReset](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			Services->GetStatsInterface()->ResetStats({AccountId}).OnComplete([Promise=MoveTemp(Promise), &ImplementedStatsReset](const TOnlineResult<FResetStats>& Result) mutable
			{
				CHECK_OP_EQ(Result, Errors::NotImplemented());

				if (Result.IsOk())
				{
					ImplementedStatsReset = true;
				}

				Promise->SetValue(true);
			});
		})
		.EmplaceAsyncLambda([&AccountId, &UserStats, &ImplementedStatsReset](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			if (!ImplementedStatsReset)
			{
				Promise->SetValue(true);
				return;
			}

			QueryStats_Fixture(MoveTemp(Promise), Services, AccountId, AccountId, { "Stat_Use_Largest", "Stat_Use_Smallest", "Stat_Use_Sum", "Stat_Use_Set" }, UserStats); 
		})
		.EmplaceLambda([&UserStats, &ImplementedStatsReset](const IOnlineServicesPtr& Services) {
			if (!ImplementedStatsReset)
			{
				return;
			}

			TSharedPtr<FStatsCommon> StatsCommon = StaticCastSharedPtr<FStatsCommon>(Services->GetStatsInterface());

			CHECK(*UserStats.Find("Stat_Use_Largest") == StatsCommon->GetStatDefinition("Stat_Use_Largest")->DefaultValue);
			CHECK(*UserStats.Find("Stat_Use_Smallest") == StatsCommon->GetStatDefinition("Stat_Use_Smallest")->DefaultValue);
			CHECK(*UserStats.Find("Stat_Use_Sum") == StatsCommon->GetStatDefinition("Stat_Use_Sum")->DefaultValue);
			CHECK(*UserStats.Find("Stat_Use_Set") == StatsCommon->GetStatDefinition("Stat_Use_Set")->DefaultValue);
		});

	RunToCompletion();
}

void BatchQueryStats_Fixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FAccountId& LocalAccountId, const TArray<FAccountId>& AccountIds, TArray<FUserStats>& OutUsersStats)
{
	TArray<FString> StatNames = { "Stat_Use_Largest", "Stat_Use_Smallest", "Stat_Use_Sum", "Stat_Use_Set" };
	Services->GetStatsInterface()->BatchQueryStats({ LocalAccountId, AccountIds, StatNames }).OnComplete([Promise=MoveTemp(Promise), &OutUsersStats](const TOnlineResult<UE::Online::FBatchQueryStats>& Result) mutable
	{
		CHECK_OP(Result);
		Promise->SetValue(true);
		OutUsersStats = Result.GetOkValue().UsersStats;
	});
}

int64 StatValueInBatchQueryResult(const TArray<FUserStats>& UsersStats, const FAccountId& AccountId, const FString& StatName)
{
	const FUserStats* UserStats = UsersStats.FindByPredicate([&AccountId](const FUserStats& UserStats) { return UserStats.AccountId == AccountId; });
	if (UserStats)
	{
		if (const FStatValue* StatValue = UserStats->Stats.Find(StatName))
		{
			return StatValue->GetInt64();
		}
	}

	return 0;
}

STATS_TEST_CASE("BatchUpdateAndQuery with 2 users", "[MultiAccount]")
{
	FAccountId AccountIdA, AccountIdB;
	TArray<FUserStats> UpdatedUsersStats;
	int64 Expected_Stat_Use_Set_A = (int64)FMath::RandRange(1, 100);
	int64 Expected_Stat_Use_Set_B = (int64)FMath::RandRange(1, 100);

	GetLoginPipeline({ AccountIdA, AccountIdB })
		.EmplaceAsyncLambda([&AccountIdA, &Expected_Stat_Use_Set_A](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			TArray<FUserStats> UpdateUsersStats{
				{ AccountIdA, { { "Stat_Use_Set", FStatValue(Expected_Stat_Use_Set_A) } } },
			};
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountIdA, UpdateUsersStats);
		})
		.EmplaceAsyncLambda([&AccountIdB, &Expected_Stat_Use_Set_B](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			TArray<FUserStats> UpdateUsersStats{
				{ AccountIdB, { { "Stat_Use_Set", FStatValue(Expected_Stat_Use_Set_B) } } },
			};
			UpdateStats_Fixture(MoveTemp(Promise), Services, AccountIdB, UpdateUsersStats);
		})
		.EmplaceAsyncLambda([&AccountIdA, &AccountIdB, &UpdatedUsersStats](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services) {
			BatchQueryStats_Fixture(MoveTemp(Promise), Services, AccountIdA, { AccountIdA, AccountIdB }, UpdatedUsersStats);
		})
		.EmplaceLambda([&AccountIdA, &AccountIdB, &UpdatedUsersStats, &Expected_Stat_Use_Set_A, &Expected_Stat_Use_Set_B](const IOnlineServicesPtr& Services) {
			CHECK_EQUAL(StatValueInBatchQueryResult(UpdatedUsersStats, AccountIdA, "Stat_Use_Set"), Expected_Stat_Use_Set_A);
			CHECK_EQUAL(StatValueInBatchQueryResult(UpdatedUsersStats, AccountIdB, "Stat_Use_Set"), Expected_Stat_Use_Set_B);
		});

	RunToCompletion();
}

// UpdateStats tests
STATS_TEST_CASE("Verify UpdateStats returns a fail message if the local user is not logged in, and makes no changes to any users' stats")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats returns a fail message of the given local user ID does not match the actual local user ID, and makes no changes to any users' stats")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats makes no changes to any users' stats if given an empty UserStats array")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats makes no changes to a user's stats if given a UserStats array that contains their user ID but the respective inner stats array is empty")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats returns a fail message if given a UserStats array that contains an invalid user ID")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats returns a fail message if given a UserStats array that contains one valid user ID and one invalid user ID, and does not update the valid user's stats")
{
	// TODO
}

//STATS_TEST_CASE("Verify UpdateStats updates a user's stats if given one stat to update") // Already covered
//{
//	// TODO
//}

STATS_TEST_CASE("Verify UpdateStats updates a user's stats if given two stats to update")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats updates a user's stats if given a stat with the same value they already have, but the value does not change")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats returns a fail message if given an invalid stat")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateState returns a fail message if given one valid stat and one invalid stat for a user, and makes no changes")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats returns a fail message if given a valid stat for one user and an invalid stat for another user, and makes no changes to either user's stats")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats updates the correct stats if given one stat for one user and a different stat for another user")
{
	// TODO
}

STATS_TEST_CASE("Verify UpdateStats updates the stats to correct values if given one value for one user and a different value for the same stat of another user")
{
	// TODO
}

// QueryStats tests
STATS_TEST_CASE("Verify QueryStats returns a fail message if the local user is not logged in")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns a fail message if the given local user ID does not match the actual local user ID")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns a fail message if given an invalid target user ID")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns an empty map if given an empty StatNames array")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns a fail message if given a StatNames array with an invalid stat")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns a fail message if given a StatNames array with one valid stat and one invalid stat")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns a map with the correct stat and value if given a StatNames array with one valid stat")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns a map with the correct stats and values if given a StatNames array with multiple valid stats")
{
	// TODO
}

STATS_TEST_CASE("Verify QueryStats returns the correct stat values for the target user if there is another user with different values for the same stats")
{
	// TODO
}

// BatchQueryStats tests
STATS_TEST_CASE("Verify BatchQueryStats returns a fail message if the local user is not logged in")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns a fail message if the given local user ID does not match the actual local user ID")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns a fail message if given a TargetUserIds array with one invalid target user ID")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns a fail message if given a TargetUserIds array with one valid target user ID and one invalid target user ID")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns an empty array if given an empty TargetUserIds array and an empty StatNames array")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns an empty array if given an empty TargetUserIds array and a populated StatNames array")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns the correct UserStats (with empty Stats arrays) if given a populated TargetUserIds array and an empty StatNames array")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns a fail message if given a StatNames array with an invalid stat")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns a fail message if given a StatNames array with one valid stat and one invalid stat")
{
	// TODO
}

STATS_TEST_CASE("Verify BatchQueryStats returns the correct UserStats for the target user when given one valid target user ID")
{
	// TODO
}

//STATS_TEST_CASE("Verify BatchQueryStats returns the correct UserStats for the target users when given multiple valid target user IDs") // Already covered
//{
//	// TODO
//}

STATS_TEST_CASE("Verify BatchQueryStats returns the correct UserStats for the target users when given one valid stat")
{
	// TODO
}

//STATS_TEST_CASE("Verify BatchQueryStats returns the correct UserStats for the target users when given multiple valid stats") // Already covered
//{
//	// TODO
//}

// ResetStats tests
//STATS_TEST_CASE("Verify ResetStats resets the stats of the given target user and displays a success message") // Already covered
//{
//	// TODO
//}

STATS_TEST_CASE("Verify ResetStats returns a fail message if the local user is not logged in")
{
	// TODO
}

STATS_TEST_CASE("Verify ResetStats returns a fail message if the given local user ID does not match the actual local user ID")
{
	// TODO
}