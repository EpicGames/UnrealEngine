// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include <catch2/catch_test_macros.hpp>

#include "OnlineCatchHelper.h"

#include "Online/OnlineUtils.h"
#include "Online/OnlineResult.h"
#include "Online/OnlineServicesCommon.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"
#include "HAL/PlatformTLS.h"

#define ASYNCOP_TAG "[AsyncOp]"
#define ASYNCOP_SKIP_EOS_PLATFORM_FAULT_TAG "[AsyncOp][skip_eos_platform_fault]"
#define ASYNCOP_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, ASYNCOP_TAG __VA_ARGS__)

struct FTestOp
{
	static constexpr TCHAR Name[] = TEXT("TestOp");

	struct Params {};
	struct Result {};
};

struct FTestOpWithParams
{
	static constexpr TCHAR Name[] = TEXT("TestOpWithParams");

	struct Params { int32 Index = -1; };
	struct Result {};
};

struct FMergeableTestOpWithParams
{
	static constexpr TCHAR Name[] = TEXT("MergeableTestOpWithParams");

	struct Params { int32 Index = -1; int32 Mutations = 0; };
	struct Result {};
};

namespace UE::Online::Meta {

BEGIN_ONLINE_STRUCT_META(FTestOp::Params)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTestOp::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTestOpWithParams::Params)
	ONLINE_STRUCT_FIELD(FTestOpWithParams::Params, Index)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTestOpWithParams::Result)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FMergeableTestOpWithParams::Params)
	ONLINE_STRUCT_FIELD(FMergeableTestOpWithParams::Params, Index),
	ONLINE_STRUCT_FIELD(FMergeableTestOpWithParams::Params, Mutations)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FMergeableTestOpWithParams::Result)
END_ONLINE_STRUCT_META()

/* namespace UE::Online::Meta */ }

ASYNCOP_TEST_CASE("Verify that the cache returns the same joinable op when the parameters are the same")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
		{
			FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
			FTestOpWithParams::Params Params1;
			Params1.Index = 0;
			TOnlineAsyncOpRef<FTestOpWithParams> Op = ServicesCommon.GetJoinableOp<FTestOpWithParams>(MoveTemp(Params1));
			CHECK(!Op->IsReady());
			Op->Then([Promise](FOnlineAsyncOp&) { Promise->SetValue(true); })
			.Enqueue(ServicesCommon.GetParallelQueue());

			FTestOpWithParams::Params Params2;
			Params2.Index = 0;
			TOnlineAsyncOpRef<FTestOpWithParams> Op2 = ServicesCommon.GetJoinableOp<FTestOpWithParams>(MoveTemp(Params2));
			CHECK(Op2->IsReady());
		});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the cache returns different joinable ops when the parameters are different")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
		{
			FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
			FTestOpWithParams::Params Params1;
			Params1.Index = 1;
			TOnlineAsyncOpRef<FTestOpWithParams> Op = ServicesCommon.GetJoinableOp<FTestOpWithParams>(MoveTemp(Params1));
			CHECK(!Op->IsReady());
			Op->Then([](FOnlineAsyncOp&) {})
			.Enqueue(ServicesCommon.GetParallelQueue());

			FTestOpWithParams::Params Params2;
			Params2.Index = 2;
			TOnlineAsyncOpRef<FTestOpWithParams> Op2 = ServicesCommon.GetJoinableOp<FTestOpWithParams>(MoveTemp(Params2));
			CHECK(!Op2->IsReady());
			Op2->Then([Promise](FOnlineAsyncOp&) { Promise->SetValue(true); })
			.Enqueue(ServicesCommon.GetParallelQueue());
		});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the cache returns the same mergeable op when the parameters are the same")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
		{
			FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
			FMergeableTestOpWithParams::Params Params1;
			Params1.Index = 0;
			TOnlineAsyncOpRef<FMergeableTestOpWithParams> Op = ServicesCommon.GetMergeableOp<FMergeableTestOpWithParams>(MoveTemp(Params1));
			CHECK(!Op->IsReady());
			Op->Then([Promise](FOnlineAsyncOp&) { Promise->SetValue(true); })
			.Enqueue(ServicesCommon.GetParallelQueue());

			FMergeableTestOpWithParams::Params Params2;
			Params2.Index = 0;
			TOnlineAsyncOpRef<FMergeableTestOpWithParams> Op2 = ServicesCommon.GetMergeableOp<FMergeableTestOpWithParams>(MoveTemp(Params2));
			CHECK(Op2->IsReady());
		});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the cache returns different mergeable ops when the parameters are different")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
		{
			FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
			FMergeableTestOpWithParams::Params Params1;
			Params1.Index = 1;
			TOnlineAsyncOpRef<FMergeableTestOpWithParams> Op = ServicesCommon.GetMergeableOp<FMergeableTestOpWithParams>(MoveTemp(Params1));
			CHECK(!Op->IsReady());
			Op->Then([](FOnlineAsyncOp&) {})
			.Enqueue(ServicesCommon.GetParallelQueue());

			FMergeableTestOpWithParams::Params Params2;
			Params2.Index = 2;
			TOnlineAsyncOpRef<FMergeableTestOpWithParams> Op2 = ServicesCommon.GetMergeableOp<FMergeableTestOpWithParams>(MoveTemp(Params2));
			CHECK(!Op2->IsReady());
			Op2->Then([Promise](FOnlineAsyncOp&) { Promise->SetValue(true); })
			.Enqueue(ServicesCommon.GetParallelQueue());
		});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the various OnlineAsyncOp continuation forms complete (Immediately fulfilled promises)")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult AsyncResultPtr, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetOp<FTestOp>(FTestOp::Params());
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([](FOnlineAsyncOp&)
		{
			return MakeFulfilledPromise<void>().GetFuture();
		})
		.Then([](FOnlineAsyncOp&)
		{
			return 1;
		} )
		.Then([](FOnlineAsyncOp&, int32)
		{
			return MakeFulfilledPromise<int32>(1).GetFuture();
		})
		.Then([](FOnlineAsyncOp&, int32, TPromise<TArray<int32>>&& Promise)
		{
			Promise.SetValue(TArray<int32>{1,2,3,4,5,6,7,8,9,0});
		})
		.ForEach([](FOnlineAsyncOp&, int32)
		{
			return 1;
		})
 		.ForEach([](FOnlineAsyncOp&, int32)
 		{
 			return MakeFulfilledPromise<int32>(1).GetFuture();
 		})
 		.ForEach([](FOnlineAsyncOp&, int32, TPromise<int32>&& Promise)
 		{
 			Promise.SetValue(1);
 		})
 		.ForEachN(3, [](FOnlineAsyncOp&, TArrayView<int32> Value)
 		{
 			return TArray<int32>(Value);
 		})
 		.ForEachN(3, [](FOnlineAsyncOp&, TArrayView<int32> Value)
 		{
 			return MakeFulfilledPromise<TArray<int32>>(Value).GetFuture();
 		})
 		.ForEachN(3, [](FOnlineAsyncOp&, TArrayView<int32> Value, TPromise<TArray<int32>>&& Promise)
 		{
 			Promise.SetValue(TArray<int32>(Value));
 		})
		.Then([AsyncResultPtr](TOnlineAsyncOp<FTestOp>& InOp, const TArray<int32>&)
		{
			InOp.SetResult(FTestOp::Result());
			AsyncResultPtr->SetValue(true);
		})
		.Enqueue(ServicesCommon.GetParallelQueue());
	});
	
	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the various OnlineAsyncOp continuation forms complete (Promises not immediately fulfilled)")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult AsyncResultPtr, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetOp<FTestOp>(FTestOp::Params());
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([](FOnlineAsyncOp&) { return Async(EAsyncExecution::Thread, []() { return; }); })
		.Then([](FOnlineAsyncOp&) { return 1; })
		.Then([](FOnlineAsyncOp&, int32) { return Async(EAsyncExecution::Thread, []() { return 1; }); })
		.Then([](FOnlineAsyncOp&, int32, TPromise<TArray<int32>>&& Promise)
		{
			Async(EAsyncExecution::Thread, []() { return 1; })
			.Next([MovedPromise = MoveTemp(Promise)](int32) mutable { MovedPromise.SetValue(TArray<int32>{1, 2, 3, 4, 5, 6, 7, 8, 9, 0}); });
		})
		.ForEach([](FOnlineAsyncOp&, int32) { return 1; })
		.ForEach([](FOnlineAsyncOp&, int32) { return Async(EAsyncExecution::Thread, []() { return 1; }); })
		.ForEach([](FOnlineAsyncOp&, int32, TPromise<int32>&& Promise)
		{
			Async(EAsyncExecution::Thread, []() { return 1; })
			.Next([MovedPromise = MoveTemp(Promise)](int32) mutable { MovedPromise.SetValue(1); });
		})
		.ForEachN(3, [](FOnlineAsyncOp&, TArrayView<int32> Value){ return TArray<int32>(Value); })
		.ForEachN(3, [](FOnlineAsyncOp&, TArrayView<int32> Value)
		{
			return Async(EAsyncExecution::Thread, [Result = TArray<int32>(Value)]() { return Result; });
		})
		.ForEachN(3, [](FOnlineAsyncOp&, TArrayView<int32> Value, TPromise<TArray<int32>>&& Promise)
		{
			Async(EAsyncExecution::Thread, [] { return 1; })
			.Next([MovedPromise = MoveTemp(Promise), Result = TArray<int32>(Value)](int32) mutable
			{
				MovedPromise.SetValue(Result);
			});
		})
		.Then([AsyncResultPtr](TOnlineAsyncOp<FTestOp>& Op, const TArray<int32>& Results)
		{
			Op.SetResult(FTestOp::Result());
			AsyncResultPtr->SetValue(true);
		})
		.Enqueue(ServicesCommon.GetParallelQueue());
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the various OnlineAsyncOp execution policies work as expected)")
{
	FTestPipeline& TestPipeline = GetPipeline();
	uint32 ThreadId;
	TestPipeline.EmplaceAsyncLambda([&ThreadId](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetOp<FTestOp>(FTestOp::Params());
		Op->Then([&ThreadId](FOnlineAsyncOp&) {})
		.Then([](FOnlineAsyncOp&)
		{
			// This is expected to be on the game thread
			CHECK((FTaskGraphInterface::Get().IsCurrentThreadKnown() && FTaskGraphInterface::Get().GetCurrentThreadIfKnown() == ENamedThreads::GameThread));
		} , FOnlineAsyncExecutionPolicy::RunOnNextTick())
		.Then([&ThreadId](FOnlineAsyncOp&)
		{
			// Task graph won't know the thread pool thread
			CHECK(!FTaskGraphInterface::Get().IsCurrentThreadKnown());
			ThreadId = FPlatformTLS::GetCurrentThreadId();
		}, FOnlineAsyncExecutionPolicy::RunOnThreadPool())
		.Then([&ThreadId](FOnlineAsyncOp&)
		{
			// because the previous step returns immediately, this should be run on the same thread as the previous step
			CHECK(ThreadId == FPlatformTLS::GetCurrentThreadId());
		}, FOnlineAsyncExecutionPolicy::RunImmediately())
		.Then([](FOnlineAsyncOp&)
		{
			// This can be on any task graph thread
			CHECK(FTaskGraphInterface::Get().IsCurrentThreadKnown());
		}, FOnlineAsyncExecutionPolicy::RunOnTaskGraph())
		.Then([](FOnlineAsyncOp&)
		{
			// This is expected to be on the game thread
			CHECK((FTaskGraphInterface::Get().IsCurrentThreadKnown() && FTaskGraphInterface::Get().GetCurrentThreadIfKnown() == ENamedThreads::GameThread));
		}, FOnlineAsyncExecutionPolicy::RunOnGameThread())
		.Then([Promise](TOnlineAsyncOp<FTestOp>& Op) { Op.SetResult(FTestOp::Result()); Promise->SetValue(true); })
		.Enqueue(ServicesCommon.GetParallelQueue());
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the duration cache policy works)")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TArray<FString> CacheDurationTestHierarchy = { TEXT("OnlineTests.TestCacheDuration") };
	// Uses the following operation configuration in OnlineTests' DefaultEngine.ini
	// [OnlineTests.TestCacheDuration]
	// CacheExpiration=Duration
	// CacheExpirySeconds=2
	// bCacheError=false
	TestPipeline.EmplaceLambda([&CacheDurationTestHierarchy](const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheDurationTestHierarchy);
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([](TOnlineAsyncOp<FTestOp>& Op) { Op.SetResult(FTestOp::Result()); })
		.Enqueue(ServicesCommon.GetParallelQueue());
	})
	.EmplaceLambda([&CacheDurationTestHierarchy](const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheDurationTestHierarchy);
		CHECK(Op->IsReady()); // Cache should return the already in flight/completed op
	})
	.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		// Wait a second into the cache duration
		FTSTicker::GetCoreTicker().AddTicker(TEXT("OnlineTestTimeout"), 1.0f, [Promise](float) { Promise->SetValue(true); return false; });
	})
	.EmplaceLambda([&CacheDurationTestHierarchy](const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheDurationTestHierarchy);
		CHECK(Op->IsReady()); // Cache should return the already completed op
	})
	.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		// Wait until shortly after the cache should expire
		FTSTicker::GetCoreTicker().AddTicker(TEXT("OnlineTestTimeout"), 2.0f, [Promise](float) { Promise->SetValue(true); return false; });
	})
	.EmplaceAsyncLambda([&CacheDurationTestHierarchy](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheDurationTestHierarchy);
		CHECK(!Op->IsReady()); // Cache should return a new op
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([Promise](TOnlineAsyncOp<FTestOp>& Op) { Op.SetResult(FTestOp::Result()); Promise->SetValue(true); })
		.Enqueue(ServicesCommon.GetParallelQueue());
	})
	.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		// Wait until shortly after the cache should expire
		FTSTicker::GetCoreTicker().AddTicker(TEXT("OnlineTestTimeout"), 3.0f, [Promise](float) { Promise->SetValue(true); return false; });
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the cache error setting works)", ASYNCOP_SKIP_EOS_PLATFORM_FAULT_TAG)
{
	FTestPipeline& TestPipeline = GetPipeline();
	TArray<FString> DontCacheErrorHierarchy = { TEXT("OnlineTests.TestDontCacheError") };
	// Uses the following operation configuration in OnlineTests' DefaultEngine.ini
	// [OnlineTests.TestDontCacheError]
	// CacheExpiration=Duration
	// CacheExpirySeconds=1
	// bCacheError=false
	TArray<FString> CacheErrorHierarchy = { TEXT("OnlineTests.TestCacheError") };
	// Uses the following operation configuration in OnlineTests' DefaultEngine.ini
	// [OnlineTests.TestCacheError]
	// CacheExpiration=Duration
	// CacheExpirySeconds=1
	// bCacheError=true
	TestPipeline.EmplaceAsyncLambda([&DontCacheErrorHierarchy](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), DontCacheErrorHierarchy);
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([Promise](TOnlineAsyncOp<FTestOp>& Op) { Op.SetError(Errors::Unknown()); Promise->SetValue(true); })
		.Enqueue(ServicesCommon.GetParallelQueue());
	})
	.EmplaceAsyncLambda([&CacheErrorHierarchy](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheErrorHierarchy);
		CHECK(!Op->IsReady()); // Should return a new op
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([Promise](TOnlineAsyncOp<FTestOp>& Op) { Op.SetError(Errors::Unknown()); Promise->SetValue(true); })
		.Enqueue(ServicesCommon.GetParallelQueue());
	})
	.EmplaceAsyncLambda([&CacheErrorHierarchy](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheErrorHierarchy);
		CHECK(Op->IsReady()); // Should return the cached, failed op
		Op->GetHandle().OnComplete([Promise](const TOnlineResult<FTestOp>& Result)
		{
			CHECK(Result.IsError());
			Promise->SetValue(true);
		});
	})
	.EmplaceAsyncLambda([](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		// Wait until shortly after the cache should expire
		FTSTicker::GetCoreTicker().AddTicker(TEXT("OnlineTestTimeout"), 2.0f, [Promise](float) { Promise->SetValue(true); return false; });
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that the cache expires once complete)")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TArray<FString> CacheExpireUponCompletionHierarchy = { TEXT("OnlineTests.TestCacheExpireUponCompletion") };
	// Uses the following operation configuration in OnlineTests' DefaultEngine.ini
	// [OnlineTests.TestCacheExpireUponCompletion]
	// CacheExpiration=UponCompletion
	// bCacheError=false
	TestPipeline.EmplaceAsyncLambda([&CacheExpireUponCompletionHierarchy](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheExpireUponCompletionHierarchy);
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([Promise](TOnlineAsyncOp<FTestOp>& Op) { Op.SetResult(FTestOp::Result()); Promise->SetValue(true); })
		.Enqueue(ServicesCommon.GetParallelQueue());
		TOnlineAsyncOpRef<FTestOp> Op2 = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheExpireUponCompletionHierarchy);
		CHECK(Op2->IsReady());
		CHECK(!Op2->IsComplete());
	})
	.EmplaceAsyncLambda([&CacheExpireUponCompletionHierarchy](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetJoinableOp<FTestOp>(FTestOp::Params(), CacheExpireUponCompletionHierarchy);
		CHECK(!Op->IsReady());
		Op->Then([](FOnlineAsyncOp&) {})
		.Then([Promise](TOnlineAsyncOp<FTestOp>& Op) { Op.SetResult(FTestOp::Result()); Promise->SetValue(true); })
		.Enqueue(ServicesCommon.GetParallelQueue());
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that wrapped handle completion delegate is called")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult AsyncResultPtr, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOpWithParams> Op = ServicesCommon.GetOp<FTestOpWithParams>(FTestOpWithParams::Params({1}));
		Op->Then([](TOnlineAsyncOp<FTestOpWithParams>& Op) { Op.SetResult(FTestOpWithParams::Result()); })
		.Enqueue(ServicesCommon.GetParallelQueue());

		Op->GetWrappedHandle<FTestOp>([](const TOnlineResult<FTestOpWithParams>& Result)
			{
				if (Result.IsOk())
				{
					return TOnlineResult<FTestOp>(FTestOp::Result());
				}
				else
				{
					return TOnlineResult<FTestOp>(Result.GetErrorValue());
				}
			}).OnComplete([AsyncResultPtr](const TOnlineResult<FTestOp>& Result)
			{
				CHECK(Result.IsOk());
				AsyncResultPtr->SetValue(true);
			});
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that wrapped handle completion delegate is called in error case")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult AsyncResultPtr, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOpWithParams> Op = ServicesCommon.GetOp<FTestOpWithParams>(FTestOpWithParams::Params({1}));
		Op->Then([](TOnlineAsyncOp<FTestOpWithParams>& Op) { Op.SetError(UE::Online::Errors::Unknown()); })
		.Enqueue(ServicesCommon.GetParallelQueue());

		Op->GetWrappedHandle<FTestOp>([](const TOnlineResult<FTestOpWithParams>& Result)
			{
				if (Result.IsOk())
				{
					return TOnlineResult<FTestOp>(FTestOp::Result());
				}
				else
				{
					return TOnlineResult<FTestOp>(Result.GetErrorValue());
				}
			}).OnComplete([AsyncResultPtr](const TOnlineResult<FTestOp>& Result)
			{
				CHECK(Result.IsError());
				AsyncResultPtr->SetValue(true);
			});
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that wrapped handle completion delegate is called with error in transformed error case")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TestPipeline.EmplaceAsyncLambda([](FAsyncLambdaResult AsyncResultPtr, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOpWithParams> Op = ServicesCommon.GetOp<FTestOpWithParams>(FTestOpWithParams::Params({1}));
		Op->Then([](TOnlineAsyncOp<FTestOpWithParams>& Op) { Op.SetResult(FTestOpWithParams::Result()); })
		.Enqueue(ServicesCommon.GetParallelQueue());

		Op->GetWrappedHandle<FTestOp>([](const TOnlineResult<FTestOpWithParams>& Result)
			{
				return TOnlineResult<FTestOp>(UE::Online::Errors::Unknown());
			}).OnComplete([AsyncResultPtr](const TOnlineResult<FTestOp>& Result)
			{
				CHECK(Result.IsError());
				AsyncResultPtr->SetValue(true);
			});
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that Then called from inside a running step appends and runs the new step")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TSharedPtr<TArray<int32>> ExecutionOrder = MakeShared<TArray<int32>>();
	TestPipeline.EmplaceAsyncLambda([ExecutionOrder](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetOp<FTestOp>(FTestOp::Params());

		// Only one step is chained before Enqueue. It appends two more steps to the op while running.
		Op->Then([ExecutionOrder, Promise](TOnlineAsyncOp<FTestOp>& InAsyncOp)
		{
			ExecutionOrder->Add(1);

			InAsyncOp.Then([ExecutionOrder](TOnlineAsyncOp<FTestOp>& InAsyncOp)
			{
				ExecutionOrder->Add(2);
			})
			.Then([ExecutionOrder](TOnlineAsyncOp<FTestOp>& InAsyncOp)
			{
				ExecutionOrder->Add(3);
				InAsyncOp.SetResult(FTestOp::Result());
			});
		})
		.Enqueue(ServicesCommon.GetParallelQueue());
		Op->GetHandle().OnComplete([ExecutionOrder, Promise](const TOnlineResult<FTestOp>& Result)
			{
				CHECK(*ExecutionOrder == TArray<int32>({ 1, 2, 3 }));
				Promise->SetValue(true);
			});
	});

	RunToCompletion();
}

ASYNCOP_TEST_CASE("Verify that Then appended from inside a running step is enqueued at the tail (FIFO order)")
{
	FTestPipeline& TestPipeline = GetPipeline();
	TSharedPtr<TArray<int32>> ExecutionOrder = MakeShared<TArray<int32>>();
	TestPipeline.EmplaceAsyncLambda([ExecutionOrder](FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services)
	{
		FOnlineServicesCommon& ServicesCommon = static_cast<FOnlineServicesCommon&>(*Services);
		TOnlineAsyncOpRef<FTestOp> Op = ServicesCommon.GetOp<FTestOp>(FTestOp::Params());

		// Pre-built chain: A -> B. A appends C -> D while it runs.
		// AddStep appends to the tail of the Steps array, so the expected order is A, B, C, D
		// (not A, C, D, B). The appended steps run after any step already queued past A.
		Op->Then([ExecutionOrder, Promise](TOnlineAsyncOp<FTestOp>& InAsyncOp)
		{
			ExecutionOrder->Add(1); // A

			InAsyncOp.Then([ExecutionOrder](TOnlineAsyncOp<FTestOp>& InAsyncOp)
			{
				ExecutionOrder->Add(3); // C
			})
			.Then([ExecutionOrder](TOnlineAsyncOp<FTestOp>& InAsyncOp)
			{
				ExecutionOrder->Add(4); // D
				InAsyncOp.SetResult(FTestOp::Result());
			});
		})
		.Then([ExecutionOrder](TOnlineAsyncOp<FTestOp>& InAsyncOp)
		{
			ExecutionOrder->Add(2); // B - runs before the dynamically appended C and D
		})
		.Enqueue(ServicesCommon.GetParallelQueue());

		Op->GetHandle().OnComplete([ExecutionOrder, Promise](const TOnlineResult<FTestOp>& Result)
			{
				CHECK(*ExecutionOrder == TArray<int32>({ 1, 2, 3, 4 }));
				Promise->SetValue(true);
			});
	});

	RunToCompletion();
}

