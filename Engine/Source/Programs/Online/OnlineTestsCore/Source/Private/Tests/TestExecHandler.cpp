// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include <catch2/catch_test_macros.hpp>

#include "Online/OnlineExecHandler.h"
#include "OnlineCatchHelper.h"

#define EH_ERROR_TAG "[ExecHandler]"
#define EH_ERROR_TEST_CASE(x, ...) TEST_CASE(x, EH_ERROR_TAG __VA_ARGS__)

EH_ERROR_TEST_CASE("Confirm exec handler works for four new added types (sharedptr/ref/optional/variant)")
{
	using namespace UE::Online;

	TSharedPtr<int32> IntPtr;
	TSharedPtr<bool> BoolPtr = MakeShared<bool>(false);
	TSharedRef<FString> StringRef = MakeShared<FString>(TEXT("test1"));
	TOptional<int32> IntOptional;
	TOptional<int32> IntOptional2;
	TVariant<FString, int64, double, bool> OnlineVariantStr;
	TVariant<FString, int64, double, bool> OnlineVariantBool;
	TVariant<FString, int64, double, bool> OnlineVariantInt;
	TVariant<FString, int64, double, bool> OnlineVariantDouble;

	const TCHAR* Cmd = TEXT("503 true test2 null 57 s:helloworld b:true i:550 d:5.30");

	Private::ParseOnlineExecParams(Cmd, IntPtr);
	Private::ParseOnlineExecParams(Cmd, BoolPtr);
	Private::ParseOnlineExecParams(Cmd, StringRef);
	Private::ParseOnlineExecParams(Cmd, IntOptional);
	Private::ParseOnlineExecParams(Cmd, IntOptional2);
	Private::ParseOnlineExecParams(Cmd, OnlineVariantStr);
	Private::ParseOnlineExecParams(Cmd, OnlineVariantBool);
	Private::ParseOnlineExecParams(Cmd, OnlineVariantInt);
	Private::ParseOnlineExecParams(Cmd, OnlineVariantDouble);

	CHECK(IntPtr.IsValid());
	if (IntPtr.IsValid())
	{
		CHECK(*IntPtr == 503);
	}

	CHECK(BoolPtr.IsValid());
	if (BoolPtr.IsValid())
	{
		CHECK(*BoolPtr == true);
	}

	CHECK(StringRef->Equals(TEXT("test2")));
	CHECK(!IntOptional.IsSet());
	CHECK(IntOptional2.IsSet());
	if (IntOptional2.IsSet())
	{
		CHECK(IntOptional2.GetValue() == 57);
	}

	CHECK(OnlineVariantStr.IsType<FString>());
	if (OnlineVariantStr.IsType<FString>())
	{
		CHECK(OnlineVariantStr.Get<FString>().Equals(TEXT("helloworld")));
	}

	CHECK(OnlineVariantInt.IsType<int64>());
	if (OnlineVariantInt.IsType<int64>())
	{
		CHECK(OnlineVariantInt.Get<int64>() == 550);
	}

	CHECK(OnlineVariantDouble.IsType<double>());
	if (OnlineVariantDouble.IsType<double>())
	{
		CHECK(FMath::Abs(OnlineVariantDouble.Get<double>() - 5.30) < 0.01);
	}

	CHECK(OnlineVariantBool.IsType<bool>());
	if (OnlineVariantBool.IsType<bool>())
	{
		CHECK(OnlineVariantBool.Get<bool>() == true);
	}
}

enum class EExecTestEnum
{
	Value1,
	Value2,
	Value3
};

void LexFromString(EExecTestEnum& OutStatus, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Value1")) == 0)
	{
		OutStatus = EExecTestEnum::Value1;
	}
	else if (FCString::Stricmp(InStr, TEXT("Value2")) == 0)
	{
		OutStatus = EExecTestEnum::Value2;
	}
	else
	{
		OutStatus = EExecTestEnum::Value3;
	}
}

MAKE_VARIANT_ENUM_INFO(EExecTestEnum);


enum class EExecTestEnum2
{
	Value1,
	Value2,
	Value3
};

void LexFromString(EExecTestEnum2& OutStatus, const TCHAR* InStr)
{
	if (FCString::Stricmp(InStr, TEXT("Value1")) == 0)
	{
		OutStatus = EExecTestEnum2::Value1;
	}
	else if (FCString::Stricmp(InStr, TEXT("Value2")) == 0)
	{
		OutStatus = EExecTestEnum2::Value2;
	}
	else
	{
		OutStatus = EExecTestEnum2::Value3;
	}
}

MAKE_VARIANT_ENUM_INFO_SHORTNAME(EExecTestEnum2, etstr);



EH_ERROR_TEST_CASE("Confirm exec handler works for custom types declared with MAKE_VARIANT_ENUM_INFO")
{
	using namespace UE::Online;

	TVariant<int64, EExecTestEnum> TestVariant;
	TVariant<int64, EExecTestEnum2> TestVariant2;
	const TCHAR* Cmd = TEXT("EExecTestEnum:Value2 e:Value3");

	Private::ParseOnlineExecParams(Cmd, TestVariant);
	Private::ParseOnlineExecParams(Cmd, TestVariant2);

	CHECK(TestVariant.IsType<EExecTestEnum>());
	if (TestVariant.IsType<EExecTestEnum>())
	{
		CHECK(TestVariant.Get<EExecTestEnum>() == EExecTestEnum::Value2);
	}

	CHECK(TestVariant2.IsType<EExecTestEnum2>());
	if (TestVariant2.IsType<EExecTestEnum2>())
	{
		CHECK(TestVariant2.Get<EExecTestEnum2>() == EExecTestEnum2::Value3);
	}
}

EH_ERROR_TEST_CASE("Confirm ParseTokenExt working properly")
{
	const TCHAR* Cmd = TEXT("503 500 {Test1 Test2 {Test3 Test4]} Test5} 593");

	FString TokenA, TokenB, TokenC, TokenD, InnerTokenA, InnerTokenB, InnerTokenC, InnerInnerTokenA, InnerInnerTokenB;

	UE::Online::Private::ParseTokenExt(Cmd, TokenA, true);
	UE::Online::Private::ParseTokenExt(Cmd, TokenB, true);
	UE::Online::Private::ParseTokenExt(Cmd, TokenC, true);
	UE::Online::Private::ParseTokenExt(Cmd, TokenD, true);

	CHECK(TokenA.Equals(TEXT("503")));
	CHECK(TokenB.Equals(TEXT("500")));
	CHECK(TokenC.Equals(TEXT("Test1 Test2 {Test3 Test4]} Test5")));
	CHECK(TokenD.Equals(TEXT("593")));

	const TCHAR* TokenCPtr = *TokenC;
	UE::Online::Private::ParseTokenExt(TokenCPtr, InnerTokenA, true);
	UE::Online::Private::ParseTokenExt(TokenCPtr, InnerTokenB, true);
	UE::Online::Private::ParseTokenExt(TokenCPtr, InnerTokenC, true);

	CHECK(InnerTokenA.Equals(TEXT("Test1")));
	CHECK(InnerTokenB.Equals(TEXT("Test2")));
	CHECK(InnerTokenC.Equals(TEXT("Test3 Test4]")));

	const TCHAR* InnerTokenCPtr = *InnerTokenC;
	UE::Online::Private::ParseTokenExt(InnerTokenCPtr, InnerInnerTokenA, true);
	UE::Online::Private::ParseTokenExt(InnerTokenCPtr, InnerInnerTokenB, true);

	CHECK(InnerInnerTokenA.Equals(TEXT("Test3")));
	CHECK(InnerInnerTokenB.Equals(TEXT("Test4]")));
}

struct FTestObjectStruct
{
public:
	int32 Number;
	bool BooleanVal;
	FString StringVal;
};

namespace UE::Online::Meta
{
	BEGIN_ONLINE_STRUCT_META(FTestObjectStruct)
		ONLINE_STRUCT_FIELD(FTestObjectStruct, Number),
		ONLINE_STRUCT_FIELD(FTestObjectStruct, BooleanVal),
		ONLINE_STRUCT_FIELD(FTestObjectStruct, StringVal)
	END_ONLINE_STRUCT_META()
}

EH_ERROR_TEST_CASE("Confirm new array parsing working properly")
{
	const TCHAR* Cmd =  TEXT("[5 7 3 -1 9]");
	const TCHAR* Cmd2 = TEXT("[asdf \"hello world\" zxcv]");
	const TCHAR* Cmd3 = TEXT("[5, 7, 3, -1, 9]");
	const TCHAR* Cmd4 = TEXT("[[5,3,1], [2,1,3]]");
	const TCHAR* Cmd5 = TEXT("[{53 true hello}, {100 false \"goodbye john\"}]");

	TArray<int32> IntArr, IntArr2;
	TArray<FString> StringArr;
	TArray<TArray<int32>> NestedArr;
	TArray<FTestObjectStruct> ObjectArr;

	UE::Online::Private::ParseOnlineExecParams(Cmd, IntArr);
	UE::Online::Private::ParseOnlineExecParams(Cmd2, StringArr);
	UE::Online::Private::ParseOnlineExecParams(Cmd3, IntArr2);
	UE::Online::Private::ParseOnlineExecParams(Cmd4, NestedArr);
	UE::Online::Private::ParseOnlineExecParams(Cmd5, ObjectArr);

	CHECK(IntArr.Num() == 5);
	if (IntArr.Num() == 5)
	{
		CHECK(IntArr[0] == 5);
		CHECK(IntArr[1] == 7);
		CHECK(IntArr[2] == 3);
		CHECK(IntArr[3] == -1);
		CHECK(IntArr[4] == 9);
	}

	CHECK(IntArr2.Num() == 5);
	if (IntArr2.Num() == 5)
	{
		CHECK(IntArr2[0] == 5);
		CHECK(IntArr2[1] == 7);
		CHECK(IntArr2[2] == 3);
		CHECK(IntArr2[3] == -1);
		CHECK(IntArr2[4] == 9);
	}

	CHECK(StringArr.Num() == 3);
	if (StringArr.Num() == 3)
	{
		CHECK(StringArr[0].Equals(TEXT("asdf")));
		CHECK(StringArr[1].Equals(TEXT("hello world")));
		CHECK(StringArr[2].Equals(TEXT("zxcv")));
	}
	
	CHECK(NestedArr.Num() == 2);
	if(NestedArr.Num() == 2)
	{
		CHECK(NestedArr[0].Num() == 3);
		if (NestedArr[0].Num() == 3)
		{
			CHECK(NestedArr[0][0] == 5);
			CHECK(NestedArr[0][1] == 3);
			CHECK(NestedArr[0][2] == 1);
		}

		CHECK(NestedArr[1].Num() == 3);
		if (NestedArr[1].Num() == 3)
		{
			CHECK(NestedArr[1][0] == 2);
			CHECK(NestedArr[1][1] == 1);
			CHECK(NestedArr[1][2] == 3);
		}
	}

	CHECK(ObjectArr.Num() == 2);
	if (ObjectArr.Num() == 2)
	{
		CHECK(ObjectArr[0].Number == 53);
		CHECK(ObjectArr[0].BooleanVal == true);
		CHECK(ObjectArr[0].StringVal.Equals("hello"));

		CHECK(ObjectArr[1].Number == 100);
		CHECK(ObjectArr[1].BooleanVal == false);
		CHECK(ObjectArr[1].StringVal.Equals("goodbye john"));

	}
}

struct FTestObjectStructComplex
{
public:
	TArray<FString> StringArr;
	TOptional<TVariant<int32, bool>> OptionalVariantA;
	TOptional<TVariant<FString, double>> OptionalVariantB;
	FTestObjectStruct NestedObject;
};

namespace UE::Online::Meta
{
	BEGIN_ONLINE_STRUCT_META(FTestObjectStructComplex)
		ONLINE_STRUCT_FIELD(FTestObjectStructComplex, StringArr),
		ONLINE_STRUCT_FIELD(FTestObjectStructComplex, OptionalVariantA),
		ONLINE_STRUCT_FIELD(FTestObjectStructComplex, OptionalVariantB),
		ONLINE_STRUCT_FIELD(FTestObjectStructComplex, NestedObject)
	END_ONLINE_STRUCT_META()
}
EH_ERROR_TEST_CASE("Test conplex object parsing")
{
	const TCHAR* Cmd = TEXT("{[hello \"big world\" hi] null d:5.352 {101 false \"lorem ipsum\"}}");
	FTestObjectStructComplex ComplexObject;

	UE::Online::Private::ParseOnlineExecParams(Cmd, ComplexObject);
	CHECK(ComplexObject.StringArr.Num() == 3);
	if (ComplexObject.StringArr.Num() == 3)
	{
		CHECK(ComplexObject.StringArr[0].Equals("hello"));
		CHECK(ComplexObject.StringArr[1].Equals("big world"));
		CHECK(ComplexObject.StringArr[2].Equals("hi"));
	}

	CHECK(!ComplexObject.OptionalVariantA.IsSet());
	CHECK(ComplexObject.OptionalVariantB.IsSet());
	if (ComplexObject.OptionalVariantB.IsSet())
	{
		CHECK(ComplexObject.OptionalVariantB.GetValue().IsType<double>());
		if (ComplexObject.OptionalVariantB.GetValue().IsType<double>())
		{
			CHECK(FMath::Abs(ComplexObject.OptionalVariantB.GetValue().Get<double>() - 5.352) < 0.01);
		}
	}

	CHECK(ComplexObject.NestedObject.Number == 101);
	CHECK(ComplexObject.NestedObject.BooleanVal == false);
	CHECK(ComplexObject.NestedObject.StringVal.Equals(TEXT("lorem ipsum")));

}

ONLINE_TEST_CASE("Test new account ID support working", EH_ERROR_TAG)
{
	FAccountId Id;
	GetLoginPipeline({ Id });


	const TCHAR* Cmd = TEXT("0");
	FAccountId ParsedHandle;
	UE::Online::Private::ParseOnlineExecParams(Cmd, ParsedHandle, &*GetServices());
	CHECK(ParsedHandle == Id);

	FString LogStr = FString::Printf(TEXT("%s:%d"), LexToString(Id.GetOnlineServicesType()), Id.GetHandle());
	FString Cmd2Str = FString::Printf(TEXT("%s"), *LogStr);
	const TCHAR* Cmd2 = *Cmd2Str;
	FAccountId ParsedHandle2;
	UE::Online::Private::ParseOnlineExecParams(Cmd2, ParsedHandle2, &*GetServices());
	CHECK(ParsedHandle2 == Id);


}