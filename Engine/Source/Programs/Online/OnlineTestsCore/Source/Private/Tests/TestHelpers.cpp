// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHelpers.h"

FScopeDisableWarningsInLog::FScopeDisableWarningsInLog(FLogCategoryBase* InLogPtr)
{
	LogPtr = InLogPtr;
	OldLogLevel = LogPtr->GetVerbosity();
	LogPtr->SetVerbosity(ELogVerbosity::Error);
}

FScopeDisableWarningsInLog::~FScopeDisableWarningsInLog()
{
	LogPtr->SetVerbosity(OldLogLevel);
}