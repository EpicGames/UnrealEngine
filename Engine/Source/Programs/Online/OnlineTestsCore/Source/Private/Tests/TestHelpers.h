// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogCategory.h"
#include "Logging/LogVerbosity.h"

class FScopeDisableWarningsInLog
{
public:
	FScopeDisableWarningsInLog(FLogCategoryBase* InLogPtr);
	~FScopeDisableWarningsInLog();

private:
	FLogCategoryBase* LogPtr;
	ELogVerbosity::Type OldLogLevel;
};