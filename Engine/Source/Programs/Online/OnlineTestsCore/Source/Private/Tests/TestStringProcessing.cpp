// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include <catch2/catch_test_macros.hpp>

#include "Online/OnlineExecHandler.h"
#include "OnlineCatchHelper.h"

#if UE_ENABLE_ICU

#include "Internationalization/BreakIterator.h"

#define STRING_PROCESSING_TAG "[StringProcessing]"
#define STRING_PROCESSING_TEST_CASE(x, ...) TEST_CASE(x, STRING_PROCESSING_TAG __VA_ARGS__)

STRING_PROCESSING_TEST_CASE("Verify that player nick name length counting works well")
{
	REQUIRE(FInternationalization::Get().IsInitialized());

	// Café
	FString StringToCheck(TEXTVIEW("\u0043\u0061\u0066\u0065\u0301"));

	TSharedRef<IBreakIterator> GraphemeBreakIterator = FBreakIterator::CreateCharacterBoundaryIterator();
	GraphemeBreakIterator->SetString(StringToCheck);
	GraphemeBreakIterator->ResetToBeginning();

	int32 Count = 0;
	for (int32 CurrentCharIndex = GraphemeBreakIterator->MoveToNext(); CurrentCharIndex != INDEX_NONE; CurrentCharIndex = GraphemeBreakIterator->MoveToNext())
	{
		Count++;
	}

	// Although there are 5 code points, the number of graphemes is 4
	CHECK(StringToCheck.Len() == 5);
	CHECK(Count == 4);
}

#endif