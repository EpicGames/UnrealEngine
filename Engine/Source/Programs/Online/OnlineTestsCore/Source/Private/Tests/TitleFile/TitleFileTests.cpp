// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/TitleFile/AsyncTitleFileEnumerateFiles.h"
#include "Helpers/TitleFile/AsyncTitleFileGetEnumeratedFiles.h"
#include "Helpers/TitleFile/AsyncTitleFileReadFile.h"
#include "OnlineCatchHelper.h"
#include "Online/TitleFile.h"

#define TITLE_FILE_SUITE_TAGS "[titlefile]"
#define TITLE_FILE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, TITLE_FILE_SUITE_TAGS __VA_ARGS__)

#define TITLE_FILE_ENUMERATE_TAG "[enumeratefiles]"
#define TITLE_FILE_GET_ENUMERATED_TAG "[getenumeratedfiles]"
#define TITLE_FILE_GET_ENUMERATED_SKIP_EOS_PLATFORM_FAULT_TAG "[getenumeratedfiles][skip_eos_platform_fault]"
#define TITLE_FILE_READ_TAG "[readfile]"

// These variables must match what is defined in the backend for every implementation that the test is going to run on
// e.g. for Null they would be defined in DefaultEngine.ini, in the section [OnlineServices.Null.TitleFile]
#define FILE_A_CONTENTS TEXT("The quick brown fox jumps over the lazy dog")
#define FILE_B_CONTENTS TEXT("Lorem ipsum dolor sit amet")

using namespace UE::Online;

FTitleFileContentsRef FormatExpectedFileContents(FString FileContents)
{
	const FTCHARToUTF8 FileContentsUtf8(*FileContents);
	return MakeShared<FTitleFileContents>((uint8*)FileContentsUtf8.Get(), FileContentsUtf8.Length());
}

TITLE_FILE_TEST_CASE("EnumerateFiles", TITLE_FILE_ENUMERATE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params Params;
	Params.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncTitleFileEnumerateFiles>(MoveTemp(Params));

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("EnumerateFiles - Invalid User", TITLE_FILE_ENUMERATE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params Params;

	Pipe.EmplaceStep<FAsyncTitleFileEnumerateFiles>(MoveTemp(Params), Errors::InvalidUser());

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("GetEnumeratedFiles", TITLE_FILE_GET_ENUMERATED_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FTitleFileGetEnumeratedFiles::Params GetParams;
	GetParams.LocalAccountId = AccountId;

	FTitleFileGetEnumeratedFiles::Result GetResult;
	GetResult.Filenames = { TEXT("FileA"), TEXT("FileB") };

	Pipe.EmplaceStep<FAsyncTitleFileEnumerateFiles>(MoveTemp(EnumerateParams))
		.EmplaceStep<FAsyncTitleFileGetEnumeratedFiles>(MoveTemp(GetParams), MoveTemp(GetResult));

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("GetEnumeratedFiles - Invalid user", TITLE_FILE_GET_ENUMERATED_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FTitleFileGetEnumeratedFiles::Params GetParams;

	Pipe.EmplaceStep<FAsyncTitleFileEnumerateFiles>(MoveTemp(EnumerateParams))
		.EmplaceStep<FAsyncTitleFileGetEnumeratedFiles>(MoveTemp(GetParams), Errors::InvalidUser());

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("GetEnumeratedFiles - Invalid state", TITLE_FILE_GET_ENUMERATED_SKIP_EOS_PLATFORM_FAULT_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileGetEnumeratedFiles::Params GetParams;
	GetParams.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncTitleFileGetEnumeratedFiles>(MoveTemp(GetParams), Errors::InvalidState());

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("ReadFile", TITLE_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FTitleFileReadFile::Params ReadAParams;
	ReadAParams.LocalAccountId = AccountId;
	ReadAParams.Filename = TEXT("FileA");

	FTitleFileReadFile::Result ReadAResult { FormatExpectedFileContents(FILE_A_CONTENTS) };

	FTitleFileReadFile::Params ReadBParams;
	ReadBParams.LocalAccountId = AccountId;
	ReadBParams.Filename = TEXT("FileB");

	FTitleFileReadFile::Result ReadBResult { FormatExpectedFileContents(FILE_B_CONTENTS) };

	Pipe.EmplaceStep<FAsyncTitleFileReadFile>(MoveTemp(ReadAParams), MoveTemp(ReadAResult))
		.EmplaceStep<FAsyncTitleFileReadFile>(MoveTemp(ReadBParams), MoveTemp(ReadBResult));

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("ReadFile - Invalid User", TITLE_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FTitleFileReadFile::Params ReadParams;
	ReadParams.Filename = TEXT("FileA");

	Pipe.EmplaceStep<FAsyncTitleFileReadFile>(MoveTemp(ReadParams), Errors::InvalidUser());

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("ReadFile - Empty filename", TITLE_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileReadFile::Params ReadParams;
	ReadParams.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncTitleFileReadFile>(MoveTemp(ReadParams), Errors::InvalidParams());

	RunToCompletion();
}

TITLE_FILE_TEST_CASE("ReadFile - Not Found", TITLE_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FTitleFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FTitleFileReadFile::Params ReadParams;
	ReadParams.LocalAccountId = AccountId;
	ReadParams.Filename = TEXT("MissingFile");

	Pipe.EmplaceStep<FAsyncTitleFileReadFile>(MoveTemp(ReadParams), Errors::NotFound());

	RunToCompletion();
}