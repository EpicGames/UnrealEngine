// Copyright Epic Games, Inc. All Rights Reserved.

#include "Helpers/UserFile/AsyncUserFileEnumerateFiles.h"
#include "Helpers/UserFile/AsyncUserFileGetEnumeratedFiles.h"
#include "Helpers/UserFile/AsyncUserFileCopyFile.h"
#include "Helpers/UserFile/AsyncUserFileDeleteFile.h"
#include "Helpers/UserFile/AsyncUserFileReadFile.h"
#include "Helpers/UserFile/AsyncUserFileWriteFile.h"
#include "OnlineCatchHelper.h"
#include "Online/UserFile.h"
#include "Tests/TestHelpers.h"
#include "EOSShared.h"

#define USER_FILE_SUITE_TAGS "[userfile]"
#define USER_FILE_TEST_CASE(x, ...) ONLINE_TEST_CASE(x, USER_FILE_SUITE_TAGS __VA_ARGS__)

#define USER_FILE_ENUMERATE_TAG "[enumerate]"
#define USER_FILE_GETENUMERATED_TAG "[getenumerated]"
#define USER_FILE_WRITE_TAG "[write]"
#define USER_FILE_READ_TAG "[read]"
#define USER_FILE_COPY_TAG "[copy]"
#define USER_FILE_DELETE_TAG "[delete]"

using namespace UE::Online;

FUserFileContentsRef GetTestFileContents()
{
	const char* TestData = "The quick brown fox jumps over the lazy dog";
	return MakeShared<FUserFileContents>((uint8*)TestData, FCStringAnsi::Strlen(TestData));
}

void ClearUserFiles(FAccountId AccountId, FTestPipeline& Pipe)
{
	FUserFileDeleteFile::Params Params;
	Params.LocalAccountId = AccountId;

	TArray<TOnlineResult<FUserFileDeleteFile>> ExpectedResults;
	ExpectedResults.Emplace(FUserFileDeleteFile::Result());
	ExpectedResults.Emplace(Errors::NotFound());

	const TArray<FString> Filenames = { TEXT("TestFileA"), TEXT("TestFileB") };
	for (const FString& Filename : Filenames)
	{
		Params.Filename = Filename;
		Pipe.EmplaceStep<FAsyncUserFileDeleteFile>(CopyTemp(Params), CopyTemp(ExpectedResults));
	}
}

USER_FILE_TEST_CASE("EnumerateFiles Tests", USER_FILE_ENUMERATE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	ClearUserFiles(AccountId, Pipe);

	FUserFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncUserFileEnumerateFiles>(MoveTemp(EnumerateParams));

	RunToCompletion();
}

USER_FILE_TEST_CASE("EnumerateFiles Tests - Invalid user", USER_FILE_ENUMERATE_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileEnumerateFiles::Params EnumerateParams;

	Pipe.EmplaceStep<FAsyncUserFileEnumerateFiles>(MoveTemp(EnumerateParams), Errors::InvalidUser());

	RunToCompletion();

}

USER_FILE_TEST_CASE("GetEnumeratedFiles Tests", USER_FILE_GETENUMERATED_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	ClearUserFiles(AccountId, Pipe);

	FUserFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FUserFileGetEnumeratedFiles::Params GetParams;
	GetParams.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncUserFileEnumerateFiles>(MoveTemp(EnumerateParams))
		.EmplaceStep<FAsyncUserFileGetEnumeratedFiles>(MoveTemp(GetParams));

	RunToCompletion();
}

USER_FILE_TEST_CASE("GetEnumeratedFiles Tests - Invalid user", USER_FILE_GETENUMERATED_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileEnumerateFiles::Params EnumerateParams;
	EnumerateParams.LocalAccountId = AccountId;

	FUserFileGetEnumeratedFiles::Params GetParams;

	Pipe.EmplaceStep<FAsyncUserFileEnumerateFiles>(MoveTemp(EnumerateParams))
		.EmplaceStep<FAsyncUserFileGetEnumeratedFiles>(MoveTemp(GetParams), Errors::InvalidUser());

	RunToCompletion();
}

USER_FILE_TEST_CASE("GetEnumeratedFiles Tests - Invalid State", USER_FILE_GETENUMERATED_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileGetEnumeratedFiles::Params GetParams;
	GetParams.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncUserFileGetEnumeratedFiles>(MoveTemp(GetParams), Errors::InvalidState());

	RunToCompletion();
}

USER_FILE_TEST_CASE("WriteFile Tests", USER_FILE_WRITE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	ClearUserFiles(AccountId, Pipe);

	FUserFileWriteFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.Filename = TEXT("TestFileA");
	Params.FileContents = *GetTestFileContents();

	Pipe.EmplaceStep<FAsyncUserFileWriteFile>(MoveTemp(Params));

	RunToCompletion();
}

USER_FILE_TEST_CASE("WriteFile Tests - Empty filename", USER_FILE_WRITE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileWriteFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.FileContents = *GetTestFileContents();

	Pipe.EmplaceStep<FAsyncUserFileWriteFile>(MoveTemp(Params), Errors::InvalidParams());

	RunToCompletion();
}

USER_FILE_TEST_CASE("WriteFile Tests - Invalid User", USER_FILE_WRITE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileWriteFile::Params Params;
	Params.Filename = TEXT("TestFileA");
	Params.FileContents = *GetTestFileContents();

	Pipe.EmplaceStep<FAsyncUserFileWriteFile>(MoveTemp(Params), Errors::InvalidUser());

	RunToCompletion();
}

USER_FILE_TEST_CASE("ReadFile Tests", USER_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileWriteFile::Params WriteParams;
	WriteParams.LocalAccountId = AccountId;
	WriteParams.Filename = TEXT("TestFileA");
	WriteParams.FileContents = *GetTestFileContents();

	FUserFileReadFile::Params ReadParams;
	ReadParams.LocalAccountId = AccountId;
	ReadParams.Filename = TEXT("TestFileA");

	FUserFileReadFile::Result ReadResult { GetTestFileContents() };

	Pipe.EmplaceStep<FAsyncUserFileWriteFile>(MoveTemp(WriteParams))
		.EmplaceStep<FAsyncUserFileReadFile>(MoveTemp(ReadParams), MoveTemp(ReadResult));

	RunToCompletion();
}

USER_FILE_TEST_CASE("ReadFile Tests - Empty filename", USER_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileReadFile::Params Params;
	Params.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncUserFileReadFile>(MoveTemp(Params), Errors::InvalidParams());

	RunToCompletion();
}

USER_FILE_TEST_CASE("ReadFile Tests - Invalid User", USER_FILE_READ_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileReadFile::Params Params;
	Params.Filename = TEXT("TestFileA");

	Pipe.EmplaceStep<FAsyncUserFileReadFile>(MoveTemp(Params), Errors::InvalidUser());

	RunToCompletion();
}

USER_FILE_TEST_CASE("ReadFile Tests - Not Found", USER_FILE_READ_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileReadFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.Filename = TEXT("MissingFile");

	Pipe.EmplaceStep<FAsyncUserFileReadFile>(MoveTemp(Params), Errors::NotFound());

	RunToCompletion();
}

USER_FILE_TEST_CASE("CopyFile Tests", USER_FILE_COPY_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileWriteFile::Params WriteParams;
	WriteParams.LocalAccountId = AccountId;
	WriteParams.Filename = TEXT("TestFileA");
	WriteParams.FileContents = *GetTestFileContents();

	FUserFileCopyFile::Params CopyParams;
	CopyParams.LocalAccountId = AccountId;
	CopyParams.SourceFilename = WriteParams.Filename;
	CopyParams.TargetFilename = TEXT("TestFileB");

	FUserFileReadFile::Params ReadParams;
	ReadParams.LocalAccountId = AccountId;
	ReadParams.Filename = CopyParams.TargetFilename;

	FUserFileReadFile::Result ReadResult{ GetTestFileContents() };

	Pipe.EmplaceStep<FAsyncUserFileWriteFile>(MoveTemp(WriteParams))
		.EmplaceStep<FAsyncUserFileCopyFile>(MoveTemp(CopyParams))
		.EmplaceStep<FAsyncUserFileReadFile>(MoveTemp(ReadParams), MoveTemp(ReadResult));

	RunToCompletion();
}

USER_FILE_TEST_CASE("CopyFile Tests - Invalid User", USER_FILE_COPY_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileCopyFile::Params Params;
	Params.SourceFilename = TEXT("TestFileA");
	Params.TargetFilename = TEXT("TestFileB");

	Pipe.EmplaceStep<FAsyncUserFileCopyFile>(MoveTemp(Params), Errors::InvalidUser());

	RunToCompletion();
}

USER_FILE_TEST_CASE("CopyFile Tests - Empty target", USER_FILE_COPY_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileCopyFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.SourceFilename = TEXT("TestFileA");

	Pipe.EmplaceStep<FAsyncUserFileCopyFile>(MoveTemp(Params), Errors::InvalidParams());

	RunToCompletion();
}

USER_FILE_TEST_CASE("CopyFile Tests - Empty source", USER_FILE_COPY_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileCopyFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.TargetFilename = TEXT("TestFileB");

	Pipe.EmplaceStep<FAsyncUserFileCopyFile>(MoveTemp(Params), Errors::InvalidParams());

	RunToCompletion();
}

USER_FILE_TEST_CASE("CopyFile Tests - Not Found", USER_FILE_COPY_TAG)
{
	FScopeDisableWarningsInLog ScopeLogSDK{ &LogEOSSDK };
	FScopeDisableWarningsInLog ScopeLogOnlineSDK{ &LogOnlineServices };

	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileCopyFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.SourceFilename = TEXT("UnknownFileA");
	Params.TargetFilename = TEXT("TestFileB");

	Pipe.EmplaceStep<FAsyncUserFileCopyFile>(MoveTemp(Params), Errors::NotFound());

	RunToCompletion();
}

USER_FILE_TEST_CASE("DeleteFile Tests", USER_FILE_DELETE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileDeleteFile::Params Params;
	Params.LocalAccountId = AccountId;
	Params.Filename = TEXT("TestFileA");

	Pipe.EmplaceStep<FAsyncUserFileDeleteFile>(MoveTemp(Params));

	RunToCompletion();
}

USER_FILE_TEST_CASE("DeleteFile Tests - Invalid User", USER_FILE_DELETE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileDeleteFile::Params Params;
	Params.Filename = TEXT("TestFileA");

	Pipe.EmplaceStep<FAsyncUserFileDeleteFile>(MoveTemp(Params), Errors::InvalidUser());

	RunToCompletion();
}

USER_FILE_TEST_CASE("DeleteFile Tests - Empty filename", USER_FILE_DELETE_TAG)
{
	FAccountId AccountId;
	FTestPipeline& Pipe = GetLoginPipeline({ AccountId });

	FUserFileDeleteFile::Params Params;
	Params.LocalAccountId = AccountId;

	Pipe.EmplaceStep<FAsyncUserFileDeleteFile>(MoveTemp(Params), Errors::InvalidParams());

	RunToCompletion();
}