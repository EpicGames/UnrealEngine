// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include <catch2/catch_test_macros.hpp>

#include "Online/OnlineError.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
#include "Online/OnlineErrorDefinitions.h"

#define WIN_ERROR_TAG "[Error][Windows]"
#define WIN_ERROR_TEST_CASE(x, ...) TEST_CASE(x, WIN_ERROR_TAG __VA_ARGS__)

WIN_ERROR_TEST_CASE("Checking the error category of default Windows Error")
{
	using namespace UE::Online;

	auto AbortError = Errors::Windows::Abort();
	Errors::ErrorCodeType ExpectedCategory = Errors::ErrorCode::Category::Windows;
	Errors::ErrorCodeType ExpectedValue = Errors::ErrorValueFromHRESULT(E_ABORT); 
	Errors::ErrorCodeType ExpectedSystem = Errors::ErrorCode::System::Engine;

	CHECK(AbortError.GetCategory() == ExpectedCategory);
	CHECK(AbortError.GetValue() == ExpectedValue); 
	CHECK(AbortError.GetErrorCode() == Errors::ErrorCode::Create(ExpectedSystem, ExpectedCategory, ExpectedValue));
	CHECK(AbortError.GetErrorId() == FString::Printf(TEXT("%x.%x.%x"), ExpectedSystem, ExpectedCategory, ExpectedValue));
	CHECK(AbortError == Errors::FromHRESULT(E_ABORT));
	CHECK(AbortError == Errors::ErrorCodeFromHRESULT(E_ABORT));
}

WIN_ERROR_TEST_CASE("Checking a Windows inner with a Windows outer")
{
	using namespace UE::Online;

	auto ADError = Errors::Windows::AccessDenied(Errors::Windows::Fail());
	auto FailError = Errors::Windows::Fail();
	Errors::ErrorCodeType ExpectedCategory = Errors::ErrorCode::Category::Windows;
	Errors::ErrorCodeType ExpectedValue = Errors::ErrorValueFromHRESULT(E_ACCESSDENIED);
	Errors::ErrorCodeType ExpectedValue2 = Errors::ErrorValueFromHRESULT(E_FAIL);
	Errors::ErrorCodeType ExpectedSystem = Errors::ErrorCode::System::Engine;

	CHECK(ADError.GetCategory() == ExpectedCategory);
	CHECK(ADError.GetValue() == ExpectedValue);
	CHECK(ADError.GetErrorCode() == Errors::ErrorCode::Create(ExpectedSystem, ExpectedCategory, ExpectedValue));
	CHECK(ADError.GetErrorId() == FString::Printf(TEXT("%x.%x.%x-%x.%x.%x"), ExpectedSystem, ExpectedCategory, ExpectedValue, ExpectedSystem, ExpectedCategory, ExpectedValue2));
	CHECK(ADError == Errors::FromHRESULT(E_ACCESSDENIED));
	CHECK(ADError == Errors::FromHRESULT(E_FAIL));
	CHECK(ADError == Errors::ErrorCodeFromHRESULT(E_ACCESSDENIED));
	CHECK(ADError == Errors::ErrorCodeFromHRESULT(E_FAIL));
}

WIN_ERROR_TEST_CASE("Checking a Windows inner with a generic outer")
{
	using namespace UE::Online;

	auto ThisError = Errors::InvalidCreds(Errors::Windows::InvalidArg());
	Errors::ErrorCodeType ExpectedCategory = Errors::ErrorCode::Category::Windows;
	Errors::ErrorCodeType ExpectedValue = Errors::ErrorValueFromHRESULT(E_INVALIDARG);
	Errors::ErrorCodeType ExpectedSystem = Errors::ErrorCode::System::Engine;

	Errors::ErrorCodeType OuterExpectedCategory = Errors::ErrorCode::Category::Common;
	Errors::ErrorCodeType OuterExpectedValue = 3;
	Errors::ErrorCodeType OuterExpectedSystem = Errors::ErrorCode::System::Engine;

	CHECK(ThisError.GetInner()->GetCategory() == ExpectedCategory);
	CHECK(ThisError.GetInner()->GetValue() == ExpectedValue);
	CHECK(ThisError.GetInner()->GetErrorCode() == Errors::ErrorCode::Create(ExpectedSystem, ExpectedCategory, ExpectedValue));
	CHECK(ThisError.GetInner()->GetErrorId() == FString::Printf(TEXT("%x.%x.%x"), ExpectedSystem, ExpectedCategory, ExpectedValue));
	CHECK(ThisError.GetErrorId() == FString::Printf(TEXT("%x.%x.%x-%x.%x.%x"), OuterExpectedSystem, OuterExpectedCategory, OuterExpectedValue, ExpectedSystem, ExpectedCategory, ExpectedValue));
	CHECK(ThisError == Errors::FromHRESULT(E_INVALIDARG));
	CHECK(ThisError == Errors::ErrorCodeFromHRESULT(E_INVALIDARG));
}