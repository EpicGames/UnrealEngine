// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "TestDriver.h"
#include "Online/OnlineServicesCommon.h"

THIRD_PARTY_INCLUDES_START
#include <catch2/catch_test_macros.hpp>
THIRD_PARTY_INCLUDES_END

/*
* Class to protect unfilled promises made due to a test exiting early
* on a failed assert during a callback. This will make sure the promise created
* by this class is always filled out to at least be false.
*/
class TestSafePromiseDeleter
{
public:
	TestSafePromiseDeleter()
	{
		bPromiseCalled = false;
	}
	~TestSafePromiseDeleter()
	{
		if (!bPromiseCalled)
		{
			Promise.SetValue(false);
		}
	}
	void SetValue(bool Result)
	{
		bPromiseCalled = true;
		Promise.SetValue(Result);
	}
	TFuture<bool> GetFuture()
	{
		return Promise.GetFuture();
	}
protected:
	TPromise<bool> Promise;
	bool bPromiseCalled;
};
using FAsyncStepResult = TSharedPtr<TestSafePromiseDeleter>;

// Helper class for doing async test steps instead of normal tick-driven test steps
class FAsyncTestStep : public FTestPipeline::FStep
{
public:
	// main entry point function- OnlineSubsystem will be V2. The future bool in Result is whether to continue the test (essentially just a requirement)
	virtual void Run(FAsyncStepResult Result, const IOnlineServicesPtr& OnlineServices) = 0;
	
	virtual ~FAsyncTestStep() override
	{
		// make sure promise is fulfilled at this point
		if (!bComplete && ResultPromise && FuturePtr && !FuturePtr->IsReady())
		{
			ResultPromise->SetValue(false);
		}
	}
	
	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineServices) override
	{
		if (!bInitialized)
		{
			bInitialized = true;
			ResultPromise = MakeShared<FAsyncStepResult::ElementType>();
			FuturePtr = MakeShared<TFuture<bool>>(ResultPromise->GetFuture());
			FuturePtr->Next([this](bool bResult)
			{
				REQUIRE(bResult);
				bComplete = true;
			});

			Run(ResultPromise, OnlineServices);
		}

		if (bComplete == true)
		{
			return EContinuance::Done;
		}
		
		return EContinuance::ContinueStepping;
	}

protected:
	bool bInitialized = false;
	TAtomic<bool> bComplete = false;
	FAsyncStepResult ResultPromise{};
	TSharedPtr<TFuture<bool>> FuturePtr{};
};