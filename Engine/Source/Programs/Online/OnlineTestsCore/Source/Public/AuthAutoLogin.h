// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "AsyncTestStep.h"
#include "Online/Auth.h"
#include "Online/AuthCommon.h"
#include "Online/AuthOSSAdapter.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineServicesCommon.h"
#include "OnlineCatchHelper.h"

using namespace UE::Online;

struct FAuthAutoLoginStep : public FAsyncTestStep
{
	FAuthAutoLoginStep(int32 inLocalUserNum)
		: LocalUserNum(inLocalUserNum)
	{
	}

	virtual ~FAuthAutoLoginStep()
	{
	};

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		UE::Online::IAuthPtr OnlineAuthPtr = Services->GetAuthInterface();

		// Some auth implementations do not have an explicit login / logout. In those implementations all platform users are assumed to always be logged in.
		FPlatformUserId PlatformUserId = FPlatformUserId::CreateFromInternalId(LocalUserNum);
		TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> LocalOnlineUserResult = OnlineAuthPtr->GetLocalOnlineUserByPlatformUserId({ PlatformUserId });
		if (LocalOnlineUserResult.IsOk() && LocalOnlineUserResult.GetOkValue().AccountInfo->LoginStatus == UE::Online::ELoginStatus::LoggedIn)
		{
			CHECK_OP(LocalOnlineUserResult);
			Promise->SetValue(true);
		}
		else
		{
			UE::Online::FAuthLogin::Params LocalAccount;
			LocalAccount.PlatformUserId = PlatformUserId;
			LocalAccount.CredentialsType = UE::Online::LoginCredentialsType::Auto;
			OnlineAuthPtr->Login(MoveTemp(LocalAccount))
				.OnComplete([this, Promise = MoveTemp(Promise)](const TOnlineResult<UE::Online::FAuthLogin>& Result) {
				CHECK_OP(Result);
				Promise->SetValue(true);
			});
		}

	}

protected:
	int32 LocalUserNum = 0;
};