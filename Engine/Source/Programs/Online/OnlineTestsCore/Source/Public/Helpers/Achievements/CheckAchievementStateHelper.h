// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/Achievements.h"
#include "OnlineCatchHelper.h"

namespace UE::Online
{
	inline void CheckAchievementState_Fixture(const FAsyncLambdaResult& Promise, const IOnlineServicesPtr& Services, const FAccountId& AccountId, const FString& AchievementId, float AchievementProgress)
	{
		FQueryAchievementStates::Params Params{ AccountId };
		Services->GetAchievementsInterface()->QueryAchievementStates(MoveTemp(Params)).OnComplete([&](const TOnlineResult<UE::Online::FQueryAchievementStates>& Result) mutable
			{
				CHECK_OP(Result);
				FGetAchievementState::Params AchievementParams{ AccountId, AchievementId };
				TOnlineResult<FGetAchievementState> GetAchievementStateResult = Services->GetAchievementsInterface()->GetAchievementState(MoveTemp(AchievementParams));
				REQUIRE_OP(GetAchievementStateResult);
				CHECK(FMath::IsNearlyEqual(GetAchievementStateResult.GetOkValue().AchievementState.Progress, AchievementProgress));
				Promise->SetValue(true);
			});
	}
}
