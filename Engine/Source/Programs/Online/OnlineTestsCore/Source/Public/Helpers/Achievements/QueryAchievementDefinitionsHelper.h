// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"
#include "OnlineCatchHelper.h"

#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/Achievements.h"

struct FQueryAchievementDefinitionsHelper : public FTestPipeline::FStep
{
	using ParamsType = UE::Online::FQueryAchievementDefinitions::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FQueryAchievementDefinitions>;

	struct FHelperParams
	{
		ParamsType OpParams;
		TOptional<ResultType> ExpectedError;
	};

	FQueryAchievementDefinitionsHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
	}

	virtual ~FQueryAchievementDefinitionsHelper() = default;

	enum class EState { Init, Called, Done } State = EState::Init;

	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			REQUIRE(AchievementsInterface);

			State = EState::Called;
			AchievementsInterface->QueryAchievementDefinitions(MoveTemp(HelperParams.OpParams))
			.OnComplete([this](const ResultType& Result)
			{
				State = EState::Done;
				if (!HelperParams.ExpectedError.IsSet())
				{
					REQUIRE_OP(Result);
				}
				else
				{
					const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
					REQUIRE(ErrorValue != nullptr);
					REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
				}
			});
			break;
		}
		}

		return State == EState::Done ? EContinuance::Done : EContinuance::ContinueStepping;
	}

protected:
	FHelperParams HelperParams;

	UE::Online::IAchievementsPtr AchievementsInterface = nullptr;
};