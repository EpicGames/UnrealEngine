// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"
#include "OnlineCatchHelper.h"

#include "Online/Achievements.h"
#include "Online/Auth.h"
#include "Online/OnlineAsyncOp.h"

struct FUnlockAchievementsHelper : public FTestPipeline::FStep
{
	using ParamsType = UE::Online::FUnlockAchievements::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FUnlockAchievements>;

	struct FHelperParams
	{
		ParamsType OpParams;
		TOptional<ResultType> ExpectedError;
	};

	FUnlockAchievementsHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FUnlockAchievementsHelper() = default;

	enum class EState { Init, Called, AwaitEvent, Done } State = EState::Init;

	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			AchievementsInterface = OnlineSubsystem->GetAchievementsInterface();
			REQUIRE(AchievementsInterface);

			EventHandle = AchievementsInterface->OnAchievementStateUpdated().Add(this, &FUnlockAchievementsHelper::OnAchievementStateUpdated);

			ExpectedEventParams.LocalAccountId = HelperParams.OpParams.LocalAccountId;
			ExpectedEventParams.AchievementIds = HelperParams.OpParams.AchievementIds;

			State = EState::Called;
			AchievementsInterface->UnlockAchievements(MoveTemp(HelperParams.OpParams))
			.OnComplete([this](const ResultType& Result)
			{
				CHECK(State == EState::Called);
				if (!HelperParams.ExpectedError.IsSet())
				{
					REQUIRE_OP(Result);
				}
				else
				{
					const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
					REQUIRE(ErrorValue != nullptr);
					REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
				}

				State = Result.IsOk() && !bEventFired ? EState::AwaitEvent : EState::Done;
			});
			break;
		}
		case EState::AwaitEvent:
		{
			if (bEventFired
				// EOS has a race condition where we re-lock achievements too soon before locking them again. Roughly 1/3 of the time we never get the event callback.
				// They have stated that resetting progress is not a supported thing and there is no reliable way to know if/when enough time has passed to call Unlock, so just skip it.
				|| OnlineSubsystem->GetServicesProvider() == UE::Online::EOnlineServices::Epic)
			{
				State = EState::Done;
			}
			break;
		}
		case EState::Done:
		{
			EventHandle.Unbind();
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

	void OnAchievementStateUpdated(const UE::Online::FAchievementStateUpdated& EventParams)
	{	
		bEventFired = true;
		const bool bExpectedIsActual = EventParams == ExpectedEventParams;
		REQUIRE(bExpectedIsActual);
	}

protected:
	FHelperParams HelperParams;

	UE::Online::FAchievementStateUpdated ExpectedEventParams;
	UE::Online::FOnlineEventDelegateHandle EventHandle;
	bool bEventFired = false;

	UE::Online::IAchievementsPtr AchievementsInterface = nullptr;
};