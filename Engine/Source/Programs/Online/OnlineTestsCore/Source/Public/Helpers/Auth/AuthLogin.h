// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"
#include "AsyncTestStep.h"

#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineServicesCommon.h"
#include "OnlineCatchHelper.h"

struct FAuthLoginStep : public FAsyncTestStep
{
	FAuthLoginStep(UE::Online::FAuthLogin::Params&& InLocalAccount)
		: LocalAccount(MoveTemp(InLocalAccount))
	{
	}

	virtual ~FAuthLoginStep() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		OnlineAuthPtr = Services->GetAuthInterface();
		REQUIRE(OnlineAuthPtr);

		FPlatformUserId PlatformUserId = LocalAccount.PlatformUserId;

		TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> LocalOnlineUserResult = OnlineAuthPtr->GetLocalOnlineUserByPlatformUserId({PlatformUserId});
		if (LocalOnlineUserResult.IsOk() && LocalOnlineUserResult.GetOkValue().AccountInfo->LoginStatus == UE::Online::ELoginStatus::LoggedIn)
		{
			Promise->SetValue(true);
		}
		else
		{
			OnlineAuthPtr->Login(MoveTemp(LocalAccount))
				.OnComplete([this, PlatformUserId, Promise = MoveTemp(Promise)](const TOnlineResult<FAuthLogin> Op) mutable
				{
					CHECK_OP_EQ(Op, Errors::NotImplemented());
					if(Op.IsOk())
					{
						CHECK_OP(OnlineAuthPtr->GetLocalOnlineUserByOnlineAccountId({Op.GetOkValue().AccountInfo->AccountId}));
					}
					else if (Op.GetErrorValue() == Errors::NotImplemented())
					{
						// Some auth implementations do not have an explicit login / logout. In those implementations all platform users are assumed to always be logged in.
						TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> OnlineUserResult = OnlineAuthPtr->GetLocalOnlineUserByPlatformUserId({PlatformUserId});
						CHECK_OP(OnlineUserResult);
					}

					Promise->SetValue(true);
				});
		}
	}

protected:
	int32 LocalUserNum = 0;
	UE::Online::FAuthLogin::Params LocalAccount;
	UE::Online::IAuthPtr OnlineAuthPtr = nullptr;
};