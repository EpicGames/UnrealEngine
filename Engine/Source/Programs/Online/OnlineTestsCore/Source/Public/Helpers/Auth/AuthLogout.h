// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"
#include "TestDriver.h"
#include "TestHarness.h"
#include "Online/Auth.h"
#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineServicesCommon.h"

struct FAuthLogoutStep : public FAsyncTestStep
{
	FAuthLogoutStep(FPlatformUserId InPlatformUserId)
		: PlatformUserId(InPlatformUserId)
	{}

	FAuthLogoutStep(TSharedPtr<FPlatformUserId>&& InPlatformUserIdPtr)
		: PlatformUserIdPtr(MoveTemp(InPlatformUserIdPtr))
	{
	}

	virtual ~FAuthLogoutStep() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		OnlineAuthPtr = Services->GetAuthInterface();
		REQUIRE(OnlineAuthPtr);

		if (PlatformUserIdPtr)
		{
			PlatformUserId = *PlatformUserIdPtr;
		}
		
		UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByPlatformUserId> AccountId = OnlineAuthPtr->GetLocalOnlineUserByPlatformUserId({ PlatformUserId });

		CAPTURE(ToLogString(AccountId), PlatformUserId);
		CHECK_OP(AccountId);
		if(AccountId.IsOk())
		{
			OnlineAuthPtr->Logout({ AccountId.GetOkValue().AccountInfo->AccountId})
				.OnComplete([this, Promise = MoveTemp(Promise)](const UE::Online::TOnlineResult<UE::Online::FAuthLogout> Op) mutable
				{
					CHECK_OP_EQ(Op, Errors::NotImplemented());
					Promise->SetValue(true);
				});
		}
		else
		{
			Promise->SetValue(true);
		}
	}
protected:
	TSharedPtr<FPlatformUserId> PlatformUserIdPtr = nullptr;
	FPlatformUserId PlatformUserId;
	UE::Online::IAuthPtr OnlineAuthPtr = nullptr;
};