// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "AsyncTestStep.h"
#include "Online/AuthCommon.h"
#include "Online/Auth.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineServicesCommon.h"
#include "OnlineCatchHelper.h"

using namespace UE::Online;

struct FAuthQueryExternalAuthTokenStep : public FAsyncTestStep
{
	FAuthQueryExternalAuthTokenStep(UE::Online::FAuthQueryExternalAuthToken::Params&& InQueryParams)
		:	QueryParams(MoveTemp(InQueryParams))
	{
	}

	virtual ~FAuthQueryExternalAuthTokenStep()
	{
		if (OnlineAuthPtr != nullptr)
		{
			OnlineAuthPtr = nullptr;
		}
	};

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		OnlineAuthPtr = Services->GetAuthInterface();
		REQUIRE(OnlineAuthPtr);

		OnlineAuthPtr->QueryExternalAuthToken(MoveTemp(QueryParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const TOnlineResult<FAuthQueryExternalAuthToken>& Op) mutable
				{
					REQUIRE_OP(Op);
					const UE::Online::FExternalAuthToken ExternalAuthToken = Op.GetOkValue().ExternalAuthToken;
					REQUIRE(ExternalAuthToken.Type.IsValid());
					REQUIRE(!ExternalAuthToken.Data.IsEmpty());

					Promise->SetValue(true);
				});
	}

protected:
	int32 LocalUserNum = 0;
	UE::Online::FAuthQueryExternalAuthToken::Params QueryParams;
	UE::Online::IAuthPtr OnlineAuthPtr = nullptr;
};