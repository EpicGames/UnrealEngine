// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FCheckoutHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FCommerceCheckout::Params;
	using ResultType = UE::Online::TOnlineResult<FCommerceCheckout>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FCheckoutHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FCheckoutHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		CommerceInterface = Services->GetCommerceInterface();
		REQUIRE(CommerceInterface);

		UE::Online::TOnlineAsyncOpHandle<UE::Online::FCommerceCheckout> CheckoutResult = CommerceInterface->Checkout(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
					}
					else
					{
						CHECK(Result.GetOkValue().TransactionId.IsSet());
					}
					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ICommercePtr CommerceInterface = nullptr;
	TOptional<FString> TransactionId;
};