// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FGetEntitlementsHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FCommerceGetEntitlements::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FCommerceGetEntitlements>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FGetEntitlementsHelper(FHelperParams&& InHelperParams, const TOptional<uint32_t> InExpectedEntitlementsNum = TOptional<uint32_t>())
		: HelperParams(MoveTemp(InHelperParams))
		, ExpectedEntitlementsNum(InExpectedEntitlementsNum)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FGetEntitlementsHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		CommerceInterface = Services->GetCommerceInterface();
		REQUIRE(CommerceInterface);

		ResultType Result = CommerceInterface->GetEntitlements(MoveTemp(*HelperParams.OpParams));

		if (HelperParams.ExpectedError.IsSet())
		{
			REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
		}
		else if (ExpectedEntitlementsNum.IsSet())
		{
			CHECK(Result.GetOkValue().Entitlements.Num() == ExpectedEntitlementsNum.GetValue());
		}
		else
		{
			CHECK(Result.GetOkValue().Entitlements.IsEmpty());
		}
		Promise->SetValue(true);
	}

protected:
	FHelperParams HelperParams;
	TOptional<uint32_t> ExpectedEntitlementsNum = TOptional<uint32_t>();
	UE::Online::ICommercePtr CommerceInterface = nullptr;
	TArray<FEntitlement> Entitlements;
};