// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FGetOffersByIdHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FCommerceGetOffersById::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FCommerceGetOffersById>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FGetOffersByIdHelper(FHelperParams&& InHelperParams, const TOptional<uint32_t> InExpectedOffersNum = TOptional<uint32_t>())
		: HelperParams(MoveTemp(InHelperParams))
		, ExpectedOffersNum(InExpectedOffersNum)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FGetOffersByIdHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		CommerceInterface = Services->GetCommerceInterface();
		REQUIRE(CommerceInterface);

		ResultType Result = CommerceInterface->GetOffersById(MoveTemp(*HelperParams.OpParams));

		if (HelperParams.ExpectedError.IsSet())
		{
			REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
		}
		else if (ExpectedOffersNum.IsSet())
		{
			CHECK(Result.GetOkValue().Offers.Num() == ExpectedOffersNum.GetValue());
		}
		else
		{
			CHECK(Result.GetOkValue().Offers.IsEmpty());
		}
		Promise->SetValue(true);
	}

protected:
	FHelperParams HelperParams;
	TOptional<uint32_t> ExpectedOffersNum = TOptional<uint32_t>();
	UE::Online::ICommercePtr CommerceInterface = nullptr;
	TArray<FOffer> Offers;
};