// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FQueryOffersByIdHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FCommerceQueryOffersById::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FCommerceQueryOffersById>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FQueryOffersByIdHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FQueryOffersByIdHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		CommerceInterface = Services->GetCommerceInterface();
		REQUIRE(CommerceInterface);

		CommerceInterface->QueryOffersById(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE(Result.IsOk());
					}
					else
					{
						REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
					}
					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ICommercePtr CommerceInterface = nullptr;
};