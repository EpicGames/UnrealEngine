// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/CommerceCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"
#include "Online/OnlineServicesLog.h"

struct FRedeemEntitlementsHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FCommerceRedeemEntitlement::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FCommerceRedeemEntitlement>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FRedeemEntitlementsHelper(FHelperParams&& InHelperParams, TSharedPtr<FString> InEntitlementId)
		:	HelperParams(MoveTemp(InHelperParams))
		,	EntitlementId(InEntitlementId)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FRedeemEntitlementsHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		HelperParams.OpParams->EntitlementId = *EntitlementId;

		CommerceInterface = Services->GetCommerceInterface();
		REQUIRE(CommerceInterface);

		if (!HelperParams.ExpectedError.IsSet())
		{
			CommerceInterface->RedeemEntitlement(MoveTemp(*HelperParams.OpParams))
				.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
					{
						REQUIRE(Result.IsOk());
						Promise->SetValue(true);
					});
		}
		else
		{
			ELogVerbosity::Type OldVerbosity = LogOnlineServices.GetVerbosity();
			LogOnlineServices.SetVerbosity(ELogVerbosity::NoLogging);
			CommerceInterface->RedeemEntitlement(MoveTemp(*HelperParams.OpParams))
				.OnComplete([this, OldVerbosity, Promise = MoveTemp(Promise)](const ResultType& Result)
					{
						REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
						LogOnlineServices.SetVerbosity(OldVerbosity);
						Promise->SetValue(true);
					});
		}

	}

protected:
	FHelperParams HelperParams;
	TSharedPtr<FString> EntitlementId;
	UE::Online::ICommercePtr CommerceInterface = nullptr;
};