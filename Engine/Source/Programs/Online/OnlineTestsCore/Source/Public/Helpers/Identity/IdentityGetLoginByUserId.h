// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Online/Auth.h"
#include "Online/OnlineAsyncOpHandle.h"
#include "Online/OnlineMeta.h"
#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "OnlineSubsystem.h"

struct FIdentityGetUniquePlayerIdStep : public FTestPipeline::FStep
{
	FIdentityGetUniquePlayerIdStep(int32 InLocalUserNum, TFunction<void(UE::Online::FAccountInfo)>&& InStateSaver)
		: LocalUserNum(InLocalUserNum)
		, StateSaver(MoveTemp(InStateSaver))
	{}

	FIdentityGetUniquePlayerIdStep(int32 InLocalUserNum)
		: LocalUserNum(InLocalUserNum)
		, StateSaver([](UE::Online::FAccountInfo) {})
	{}

	virtual ~FIdentityGetUniquePlayerIdStep() = default;

	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineSubsystem) override
	{
		UE::Online::IAuthPtr OnlineAuthPtr = OnlineSubsystem->GetAuthInterface();

		UE::Online::TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByPlatformUserId> UserId = OnlineAuthPtr->GetLocalOnlineUserByPlatformUserId({ FPlatformMisc::GetPlatformUserForUserIndex(LocalUserNum) });
		REQUIRE(UserId.IsOk());
		CHECK(UserId.TryGetOkValue() != nullptr);

		StateSaver(UserId.TryGetOkValue()->AccountInfo.Get());

		return EContinuance::Done;
	}

protected:
	int32 LocalUserNum = 0;
	TFunction<void(UE::Online::FAccountInfo)> StateSaver;
};