// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

struct FLambdaStep : public FTestPipeline::FStep
{
	FLambdaStep(TFunction<void(IOnlineServicesPtr)>&& InFunction)
		: Function(MoveTemp(InFunction))
	{
	}

	virtual ~FLambdaStep() = default;

	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineSubsystem) override
	{
		Function(OnlineSubsystem);
		return EContinuance::Done;
	}

protected:
	TFunction<void(IOnlineServicesPtr)> Function;
};