// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TestDriver.h"
#include "TestHarness.h"

#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/Lobbies.h"
#include "OnlineCatchHelper.h"


struct FLobbyCreateHelper : public FTestPipeline::FStep
{
	FLobbyCreateHelper(UE::Online::FCreateLobby::Params* Params,const bool& bShouldPass = true)
		: CreateParams(Params)
		, bShouldPass(bShouldPass)
	{}

	FLobbyCreateHelper(UE::Online::FCreateLobby::Params* Params, TFunction<void(UE::Online::FLobby)>&& LobbyGetter, const bool& bShouldPass = true)
		: CreateParams(Params)
		, LobbyGetter(MoveTemp(LobbyGetter))
		, bShouldPass(bShouldPass)
		
	{}

	virtual ~FLobbyCreateHelper() = default;

	enum class EState { Init, CreateLobbyCalled, CreateLobbyInProgress, CreateLobbyComplete, Done } State = EState::Init;

	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineSubsystem) override
	{
		switch (State)
		{
		case EState::Init:
		{
			OnlineLobbiesPtr = OnlineSubsystem->GetLobbiesInterface();

			REQUIRE(OnlineLobbiesPtr != nullptr);

			State = EState::CreateLobbyCalled;
			OnlineLobbiesPtr = OnlineSubsystem->GetLobbiesInterface();
			UE::Online::TOnlineAsyncOpHandle<UE::Online::FCreateLobby> Result = OnlineLobbiesPtr->CreateLobby(MoveTemp(*CreateParams));
			Result.OnProgress([&](const UE::Online::FAsyncProgress& Prog)
			{
				State = EState::CreateLobbyInProgress;
			});
			Result.OnComplete([&](const UE::Online::TOnlineResult<UE::Online::FCreateLobby>& Result)
			{
				State = EState::CreateLobbyComplete;
				if (bShouldPass)
				{
					CHECK_OP(Result);
					CHECK((*Result.GetOkValue().Lobby).LocalName == (*CreateParams).LocalName);
					LobbyGetter(*Result.GetOkValue().Lobby);
				}
				else
				{
					CHECK(!Result.IsOk());
				}
			});
			break;
		}
		case EState::CreateLobbyInProgress:
		{
			break;
		}
		case EState::CreateLobbyComplete:
		{
			State = EState::Done;
			break;
		}
		case EState::Done:
		{
			return EContinuance::Done;
		}
		}

		return EContinuance::ContinueStepping;
	}

protected:
	UE::Online::FCreateLobby::Params* CreateParams;
	TFunction<void(UE::Online::FLobby)> LobbyGetter;
	bool bShouldPass;

	UE::Online::ILobbiesPtr OnlineLobbiesPtr = nullptr;

};
