// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"
#include "AsyncTestStep.h"
#include "Online/Lobbies.h"
#include "TestDriver.h"
#include "TestHarness.h"

#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "OnlineCatchHelper.h"


struct FLobbyJoinHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FJoinLobby::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FJoinLobby>;

	struct FHelperParams
	{
		ParamsType* OpParams;
		TOptional<ResultType> ExpectedError;
	};

	FLobbyJoinHelper(FHelperParams&& InHelperParams, const bool& bShouldPass = true)
		: HelperParams(MoveTemp(InHelperParams))
		,bShouldPass(bShouldPass)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FLobbyJoinHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		LobbyInterface = Services->GetLobbiesInterface();
		REQUIRE(LobbyInterface);

		LobbyInterface->JoinLobby(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP(Result);
					}
					else
					{
						const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
						REQUIRE(ErrorValue != nullptr);
						REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
					}

					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	bool bShouldPass;

	UE::Online::ILobbiesPtr LobbyInterface = nullptr;
};