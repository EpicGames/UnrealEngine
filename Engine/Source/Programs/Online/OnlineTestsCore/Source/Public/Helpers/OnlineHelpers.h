// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Algo/Count.h"
#include "Async/Future.h"
#include "Online/OnlineAsyncOpHandle.h"
#include "Online/OnlineError.h"
#include "Online/OnlineResult.h"
#include "Templates/ValueOrError.h"
#include "TestHarness.h"

namespace UE::TestCommon {

constexpr const int32 TickFrequencyMs = (1000 / 30); // 30 ticks per second
constexpr const int32 BlockUntilCompleteDefaultMaxTicks = (1000 * 30) / TickFrequencyMs; // Default timeout if 30 seconds

/**
 * Tick core systems that cause functionality to progress (eg core ticker)
 */
void Tick();

/** Sleep behavior for BlockUntil Complete */
enum class ESleepBehavior : uint8
{
	/** No sleeps */
	NoSleep,
	/** Sleep every tick */
	Sleep
};

/**
 * Tick core systems until a future is set, or the operation has timed out.
 * Note: Unfulfilled promises are disastrous, so this fails the test if a timeout occurs.
 * @param Futures futures to wait for
 * @param MaxTicks maximum number of ticks before deciding the operation timed out.
 * @param SleepBehavior sleep behavior while ticking
 */
template<typename ResultType>
void BlockUntilComplete(TArrayView<const TFuture<ResultType>> Futures, TOptional<int32> MaxTicks = BlockUntilCompleteDefaultMaxTicks, ESleepBehavior SleepBehavior = ESleepBehavior::NoSleep)
{
	// Tick engine systems until our future is set, or we hit a timeout limit
	int32 TicksRemaining = MaxTicks.Get(TNumericLimits<int32>::Max());

	while ((Algo::CountIf(Futures, &TFuture<ResultType>::IsValid) - Algo::CountIf(Futures, &TFuture<ResultType>::IsReady)) > 0)
	{
		Tick();
		--TicksRemaining;
		if (TicksRemaining <= 0)
		{
			REQUIRE(false);
		}
		if (SleepBehavior == ESleepBehavior::Sleep)
		{
			// Some tests take actual wall time, not just ticks to pump data through queues, such as compressing data.
			FPlatformProcess::Sleep(0.001f);
		}
	}
}

// Overload that takes a TArray of non-const futures
template<typename ResultType>
void BlockUntilComplete(const TArray<TFuture<ResultType>>& Futures, TOptional<int32> MaxTicks = BlockUntilCompleteDefaultMaxTicks, ESleepBehavior SleepBehavior = ESleepBehavior::NoSleep)
{
	BlockUntilComplete(MakeArrayView(const_cast<const TFuture<ResultType>*>(Futures.GetData()), Futures.Num()), MaxTicks, SleepBehavior);
}

// Overload that takes a single future
template<typename ResultType>
void BlockUntilComplete(const TFuture<ResultType>& Future, TOptional<int32> MaxTicks = BlockUntilCompleteDefaultMaxTicks, ESleepBehavior SleepBehavior = ESleepBehavior::NoSleep)
{
	BlockUntilComplete(MakeArrayView(&Future, 1), MaxTicks, SleepBehavior);
}

/** Block until a future is complete, and return its value */
template <typename ResultType>
ResultType GetFutureValueBlocking(TFuture<ResultType>& InFuture, TOptional<int32> MaxTicks = BlockUntilCompleteDefaultMaxTicks, ESleepBehavior SleepBehavior = ESleepBehavior::NoSleep)
{
	BlockUntilComplete(InFuture, MaxTicks, SleepBehavior);
	return InFuture.Get();
}

/** GetErrorFromResult - used by FulfillPromise to determine which error to fulfill a promise with */
// Overload that takes TOnlineResult
template<typename OpType>
[[nodiscard]] UE::Online::FOnlineError GetErrorFromResult(UE::Online::TOnlineResult<OpType>&& InResult)
{
	return InResult.IsOk() ? UE::Online::Errors::Success() : MoveTemp(InResult.GetErrorValue());
}

// Overload that takes TValueOrError where Error is FOnlineError
template<typename SuccessType>
[[nodiscard]] UE::Online::FOnlineError GetErrorFromResult(TValueOrError<SuccessType, UE::Online::FOnlineError>&& InResult)
{
	return InResult.HasValue() ? UE::Online::Errors::Success() : InResult.StealError();
}

// Overload that takes FOnlineError
[[nodiscard]] inline UE::Online::FOnlineError GetErrorFromResult(UE::Online::FOnlineError&& InError)
{
	return MoveTemp(InError);
}


/**
 * Fulfill a promise of type FOnlineError
 */
template<typename TResultType>
void FulfillPromise(TPromise<UE::Online::FOnlineError>& InPromise, TResultType&& InResult)
{
	InPromise.SetValue(GetErrorFromResult(MoveTempIfPossible(InResult)));
}

/**
 * Check that a synchronous online operation completed successfully, and return the successful result.
 * @param InResult the result
 * @return the result of the op
 */
template<typename OpType>
[[nodiscard]] OpType::Result GetOpResultChecked(UE::Online::TOnlineResult<OpType>&& InResult)
{
	REQUIRE(InResult.IsOk());
	return InResult.GetOkValue();
}

/**
 * Block until an async op completes.
 * Will fail if the async op does not complete in a timely manner.
 * @param InAsyncOp the async op to wait for
 * @return the result of the async op
 */
template<typename OpType>
UE::Online::TOnlineResult<OpType> GetAsyncOpResultBlocking(UE::Online::TOnlineAsyncOpHandle<OpType> InAsyncOp, TOptional<int32> MaxTicks = BlockUntilCompleteDefaultMaxTicks, ESleepBehavior SleepBehavior = ESleepBehavior::NoSleep)
{
	using namespace UE::Online;
	TSharedRef<TPromise<TOnlineResult<OpType>>> Promise = MakeShared<TPromise<TOnlineResult<OpType>>>();
	TFuture<TOnlineResult<OpType>> Future = Promise->GetFuture();
	InAsyncOp.OnComplete([Promise](const TOnlineResult<OpType>& Result)
	{
		Promise->EmplaceValue(Result);
	});
	return GetFutureValueBlocking(Future, MaxTicks, SleepBehavior);
}

/**
 * Block until an async op completes and returns the successful variant of the result.
 * Will fail if the async op fails, or the async op does not complete in a timely manner.
 * @param InAsyncOp the async op to wait for
 * @return the result of the async op
 */
template<typename OpType>
[[nodiscard]] OpType::Result GetAsyncOpResultChecked(UE::Online::TOnlineAsyncOpHandle<OpType> InAsyncOp, TOptional<int32> MaxTicks = BlockUntilCompleteDefaultMaxTicks, ESleepBehavior SleepBehavior = ESleepBehavior::NoSleep)
{
	return GetOpResultChecked(GetAsyncOpResultBlocking(InAsyncOp, MaxTicks, SleepBehavior));
}

/* UE::TestCommon */ }