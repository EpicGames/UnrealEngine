// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTestStep.h"
#include "Online/Presence.h"
#include "OnlineCatchHelper.h"

using namespace UE::Online;
class FBatchQueryPresenceHelper : public FAsyncTestStep
{
public:
	FBatchQueryPresenceHelper(FAccountId InLocalAccountId, TArray<FAccountId>& InTargetAccountIds, TArray<TSharedRef<const FUserPresence>>& InOutPresences, bool inBListenForChanges = false)
		: LocalAccountId(InLocalAccountId)
		, TargetAccountIds(InTargetAccountIds)
		, OutPresences(InOutPresences)
		, bListenForChanges(inBListenForChanges)
	{

	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		IPresencePtr Presence = Services->GetPresenceInterface();

		Presence->BatchQueryPresence({ LocalAccountId, TargetAccountIds, bListenForChanges })
		.OnComplete([this, Promise](const TOnlineResult<FBatchQueryPresence> Result) mutable
		{
			REQUIRE_OP(Result);
			OutPresences = Result.GetOkValue().Presences;
			Promise->SetValue(true);
		});
	}

protected:
	FAccountId LocalAccountId;
	TArray<FAccountId>& TargetAccountIds;
	TArray<TSharedRef<const FUserPresence>>& OutPresences;
	bool bListenForChanges;
};