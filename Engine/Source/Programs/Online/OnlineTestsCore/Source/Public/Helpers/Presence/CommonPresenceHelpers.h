// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTestStep.h"
#include "Online/Presence.h"
using namespace UE::Online;

inline TSharedRef<FUserPresence> GetGenericPresenceA(FAccountId AccountId)
{
	TSharedRef<FUserPresence> Presence = MakeShared<FUserPresence>();

	Presence->AccountId = AccountId;
	Presence->GameStatus = EUserPresenceGameStatus::PlayingThisGame;
	Presence->Joinability = EUserPresenceJoinability::InviteOnly;
	Presence->RichPresenceString = TEXT("[DEBUGSTR]");
	Presence->Status = EUserPresenceStatus::Online;
	Presence->StatusString = TEXT("Online");
	Presence->Properties.AddVariant(TEXT("[DEBUGKEY1]"), FString(TEXT("[DEBUGVALUE1]")));

	return Presence;
}

inline TSharedRef<FUserPresence> GetGenericPresenceB(FAccountId AccountId)
{
	TSharedRef<FUserPresence> Presence = MakeShared<FUserPresence>();

	Presence->AccountId = AccountId;
	Presence->GameStatus = EUserPresenceGameStatus::PlayingOtherGame;
	Presence->Joinability = EUserPresenceJoinability::Private;
	Presence->RichPresenceString = TEXT("[DEBUGSTR2]");
	Presence->Status = EUserPresenceStatus::ExtendedAway;
	Presence->StatusString = TEXT("Asleep");
	Presence->Properties.AddVariant(TEXT("[DEBUGKEY2]"), FString(TEXT("[DEBUGVALUE2]")));

	return Presence;
}

inline FPartialUpdatePresence::Params GetGenericPresenceMutationAtoB(FAccountId AccountId)
{
	FPartialUpdatePresence::Params Params;

	Params.LocalAccountId = AccountId;
	Params.Mutations.GameStatus = EUserPresenceGameStatus::PlayingOtherGame;
	Params.Mutations.Joinability = EUserPresenceJoinability::Private;
	Params.Mutations.RichPresenceString = TEXT("[DEBUGSTR2]");
	Params.Mutations.Status = EUserPresenceStatus::ExtendedAway;
	Params.Mutations.StatusString = TEXT("Asleep");
	Params.Mutations.RemovedProperties.Add(TEXT("[DEBUGKEY1]"));
	Params.Mutations.UpdatedProperties.AddVariant(TEXT("[DEBUGKEY2]"), FString(TEXT("[DEBUGVALUE2]")));

	return Params;
}

inline void CheckPresenceAreEqual(const FUserPresence& Presence1, const FUserPresence& Presence2)
{
	CHECK(Presence1.AccountId == Presence2.AccountId);
	CHECK(Presence1.GameStatus == Presence2.GameStatus);
	CHECK(Presence1.Joinability == Presence2.Joinability);
	CHECK(Presence1.RichPresenceString == Presence2.RichPresenceString);
	CHECK(Presence1.Status == Presence2.Status);
	CHECK(Presence1.StatusString == Presence2.StatusString);
	CHECK(Presence1.Properties.Num() == Presence2.Properties.Num());

	for (const TPair<FString, FPresenceProperty>& Pair : Presence1.Properties)
	{
		CHECK(Presence2.Properties.Contains(Pair.Key));
		if (Presence2.Properties.Contains(Pair.Key))
		{
			// TODO:  Needs operator==
			//CHECK(Pair.Value == *Presence2.Properties.Find(Pair.Key));
		}
	}
}

// don't check user ids here- may be used to check for an update on a specific user
inline void CheckPresencesNotEqual(const FUserPresence& Presence1, const FUserPresence& Presence2)
{
	CHECK(Presence1.GameStatus != Presence2.GameStatus);
	CHECK(Presence1.Joinability != Presence2.Joinability);
	CHECK(Presence1.RichPresenceString != Presence2.RichPresenceString);
	CHECK(Presence1.Status != Presence2.Status);
	CHECK(Presence1.StatusString != Presence2.StatusString);

	for (const TPair<FString, FPresenceProperty>& Pair : Presence1.Properties)
	{
		if (Presence2.Properties.Contains(Pair.Key))
		{
			// TODO:  Needs operator!=
			//CHECK(Pair.Value != *Presence2.Properties.Find(Pair.Key));
		}
	}
}

// Presence2 is static because this is intended for the first passed in presence to be something queried and the second value to be the static presence taht it is expected to be
class FComparePresencesHelper : public FAsyncTestStep
{
public:
	FComparePresencesHelper(TSharedPtr<const FUserPresence>& InPresence1, TSharedRef<const FUserPresence> InPresence2, bool inBShouldBeEqual = true)
		: Presence1(InPresence1)
		, Presence2(InPresence2)
		, bShouldBeEqual(inBShouldBeEqual)
	{

	}
	
	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		CHECK(Presence1.IsValid());

		if (Presence1.IsValid())
		{
			if (bShouldBeEqual)
			{
				CheckPresenceAreEqual(*Presence1, *Presence2);
			}
			else
			{
				CheckPresencesNotEqual(*Presence1, *Presence2);
			}
		}

		Promise->SetValue(true);
	}

protected:
	TSharedPtr<const FUserPresence>& Presence1;
	TSharedRef<const FUserPresence> Presence2;
	bool bShouldBeEqual;
};