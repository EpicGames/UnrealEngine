// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTestStep.h"
#include "Online/Presence.h"

using namespace UE::Online;
class FGetCachedPresenceHelper : public FAsyncTestStep
{
public:
	FGetCachedPresenceHelper(FAccountId InLocalAccountId, FAccountId InTargetAccountId, TSharedPtr<const FUserPresence>& InOutPresence)
		: LocalAccountId(InLocalAccountId)
		, TargetAccountId(InTargetAccountId)
		, OutPresence(InOutPresence)
	{

	}


	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		IPresencePtr Presence = Services->GetPresenceInterface();

		TOnlineResult<FGetCachedPresence> Result = Presence->GetCachedPresence({LocalAccountId, TargetAccountId});
		if (!Result.IsOk())
		{
			Promise->SetValue(false);
		}
		else 
		{
			OutPresence = Result.GetOkValue().Presence;
			Promise->SetValue(true);
		}
	}

protected:
	FAccountId LocalAccountId;
	FAccountId TargetAccountId;
	TSharedPtr<const FUserPresence>& OutPresence;
};