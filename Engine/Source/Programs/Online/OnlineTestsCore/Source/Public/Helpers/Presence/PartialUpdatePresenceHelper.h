// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTestStep.h"
#include "Online/Presence.h"
#include "OnlineCatchHelper.h"

using namespace UE::Online;
class FPartialUpdatePresenceHelper : public FAsyncTestStep
{
public:
	FPartialUpdatePresenceHelper(FPartialUpdatePresence::Params&& InParams)
		: Params(MoveTemp(InParams))
	{

	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		IPresencePtr Presence = Services->GetPresenceInterface();

		Presence->PartialUpdatePresence(MoveTemp(Params))
		.OnComplete([this, Promise](const TOnlineResult<FPartialUpdatePresence> Result) mutable
		{
			CHECK_OP(Result);
			Promise->SetValue(true);
		});
	}

protected:
	FPartialUpdatePresence::Params Params;
};