// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTestStep.h"
#include "Online/Presence.h"
#include "OnlineCatchHelper.h"

using namespace UE::Online;
class FQueryPresenceHelper : public FAsyncTestStep
{
public:
	FQueryPresenceHelper(FAccountId InLocalAccountId, FAccountId InTargetAccountId, TSharedPtr<const FUserPresence>& InOutPresence, bool inBListenToChanges = false)
		: LocalAccountId(InLocalAccountId)
		, TargetAccountId(InTargetAccountId)
		, OutPresence(InOutPresence)
		, bListenToChanges(inBListenToChanges)
	{
		
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		IPresencePtr Presence = Services->GetPresenceInterface();
		Presence->QueryPresence({LocalAccountId, TargetAccountId, bListenToChanges})
		.OnComplete([this, Promise](const TOnlineResult<FQueryPresence> Result) mutable
		{
			CHECK_OP(Result);
			if (Result.IsOk())
			{
				OutPresence = Result.GetOkValue().Presence;
			}
			Promise->SetValue(true);
		});
	}

protected:
	FAccountId LocalAccountId;
	FAccountId TargetAccountId;
	TSharedPtr<const FUserPresence>& OutPresence;
	bool bListenToChanges;
};