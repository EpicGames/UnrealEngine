// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AsyncTestStep.h"
#include "Online/Presence.h"
#include "OnlineCatchHelper.h"

using namespace UE::Online;
class FUpdatePresenceHelper : public FAsyncTestStep
{
public:
	FUpdatePresenceHelper(FAccountId InLocalAccountId, TSharedRef<FUserPresence> InPresence)
		: LocalAccountId(InLocalAccountId)
		, Presence(InPresence)
	{

	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		IPresencePtr PresencePtr = Services->GetPresenceInterface();
		PresencePtr->UpdatePresence({ LocalAccountId, Presence })
		.OnComplete([this, Promise](const TOnlineResult<FUpdatePresence> Result) mutable
		{
			CHECK_OP(Result);
			Promise->SetValue(true);
		});
	}

protected:
	FAccountId LocalAccountId;
	TSharedRef<FUserPresence> Presence;
};