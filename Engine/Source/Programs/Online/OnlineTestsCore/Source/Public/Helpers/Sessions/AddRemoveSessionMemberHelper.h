// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FAddSessionMemberHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FAddSessionMember::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FAddSessionMember>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FAddSessionMemberHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FAddSessionMemberHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		SessionsInterface->AddSessionMember(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP(Result);
					}
					else
					{
						const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
						REQUIRE(ErrorValue != nullptr);
						REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
					}

					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;

};

struct FRemoveSessionMemberHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FRemoveSessionMember::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FRemoveSessionMember>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FRemoveSessionMemberHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FRemoveSessionMemberHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		SessionsInterface->RemoveSessionMember(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
			{
				if (!HelperParams.ExpectedError.IsSet())
				{
					REQUIRE_OP(Result);
				}
				else
				{
					const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
					REQUIRE(ErrorValue != nullptr);
					REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
				}

				Promise->SetValue(true);
			});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;

};