// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"
#include "AsyncTestStep.h"

struct FCreateSessionHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FCreateSession::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FCreateSession>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FCreateSessionHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FCreateSessionHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		SessionsInterface->CreateSession(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE(Result.IsOk());
					}
					else
					{
						const UE::Online::FOnlineError* ErrorValuePtr = Result.TryGetErrorValue();
						REQUIRE(ErrorValuePtr != nullptr);
						REQUIRE(*ErrorValuePtr == HelperParams.ExpectedError->GetErrorValue());
					}
					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;

};