// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FFindSessionsHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FFindSessions::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FFindSessions>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FFindSessionsHelper(FHelperParams&& InHelperParams, const TOptional<uint32_t> InExpecetedSessionsFound = TOptional<uint32_t>(), TFunction<void()>&& InLogPrinter = []() {})
		: HelperParams(MoveTemp(InHelperParams))
		, ExpecetedSessionsFound(InExpecetedSessionsFound)
		, LogPrinter(InLogPrinter)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FFindSessionsHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{	
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		SessionsInterface->FindSessions(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP(Result);

						if (ExpecetedSessionsFound.IsSet())
						{
							if (int32 IdsNum = Result.GetOkValue().FoundSessionIds.Num())
							{
								CHECK(IdsNum == ExpecetedSessionsFound.GetValue());
							}
							else
							{
								Promise->SetValue(false);
								return;
							}
						}
						else
						{
							CHECK(Result.GetOkValue().FoundSessionIds.IsEmpty());
						}

						Promise->SetValue(true);
					}
					else
					{
						ON_SCOPE_EXIT
						{
							Promise->SetValue(true);
						};
						const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
						REQUIRE(ErrorValue != nullptr);
						REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
					}
				});

		LogPrinter();
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;
	TOptional<uint32_t> ExpecetedSessionsFound = TOptional<uint32_t>();
	TFunction<void()> LogPrinter;
};