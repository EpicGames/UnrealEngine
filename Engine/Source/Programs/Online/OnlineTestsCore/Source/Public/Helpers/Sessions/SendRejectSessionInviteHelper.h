// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FSendSessionInviteHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FSendSessionInvite::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FSendSessionInvite>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FSendSessionInviteHelper(FHelperParams&& InHelperParams, TFunction<void(const UE::Online::FSessionInviteId&)>&& InStateSaver = [](const UE::Online::FSessionInviteId&) {})
		: HelperParams(MoveTemp(InHelperParams))
		, StateSaver(InStateSaver)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FSendSessionInviteHelper()
	{
		SessionsInviteReceivedHandle.Unbind();
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		
		SessionsInviteReceivedHandle = SessionsInterface->OnSessionInviteReceived().Add([this](const UE::Online::FSessionInviteReceived& SessionsInviteReceived)
			{
				StateSaver(SessionsInviteReceived.SessionInviteId);			
			});
			
		SessionsInterface->SendSessionInvite(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP(Result);
					}
					else
					{
						const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
						REQUIRE(ErrorValue != nullptr);
						REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
					}

					Promise->SetValue(true);
				});
	}

protected:
	
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;
	TFunction<void(const UE::Online::FSessionInviteId&)> StateSaver;
	UE::Online::FOnlineEventDelegateHandle SessionsInviteReceivedHandle;
};

struct FRejectSessionInviteHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FRejectSessionInvite::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FRejectSessionInvite>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FRejectSessionInviteHelper(FHelperParams&& InHelperParams, TFunction<void(const UE::Online::FSessionInviteId&)>&& InStateSaver = [](const UE::Online::FSessionInviteId&) {})
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FRejectSessionInviteHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		SessionsInterface->RejectSessionInvite(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP(Result);
					}
					else
					{
						const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
						REQUIRE(ErrorValue != nullptr);
						REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
					}

					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;
};
