// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SessionsCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FStartMatchmakingHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FStartMatchmaking::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FStartMatchmaking>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FStartMatchmakingHelper(FHelperParams&& InHelperParams, TFunction<void(const UE::Online::FSessionInviteId&)>&& InStateSaver = [](const UE::Online::FSessionInviteId&) {})
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FStartMatchmakingHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SessionsInterface = Services->GetSessionsInterface();
		REQUIRE(SessionsInterface);

		SessionsInterface->StartMatchmaking(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE_OP(Result);
					}
					else
					{
						const UE::Online::FOnlineError* ErrorValue = Result.TryGetErrorValue();
						REQUIRE(ErrorValue != nullptr);
						REQUIRE(*ErrorValue == HelperParams.ExpectedError->GetErrorValue());
					}

					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISessionsPtr SessionsInterface = nullptr;
};
