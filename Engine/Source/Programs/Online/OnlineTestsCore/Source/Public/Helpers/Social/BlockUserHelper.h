// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SocialCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FBlockUserHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FBlockUser::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FBlockUser>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FBlockUserHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FBlockUserHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SocialInterface = Services->GetSocialInterface();
		REQUIRE(SocialInterface);

		SocialInterface->BlockUser(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE(Result.IsOk());
					}
					else
					{
						REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
					}
					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISocialPtr SocialInterface = nullptr;
};