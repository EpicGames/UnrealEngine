// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SocialCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FGetBlockedUsersHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FGetBlockedUsers::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FGetBlockedUsers>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FGetBlockedUsersHelper(FHelperParams&& InHelperParams, const TOptional<uint32_t> InExpectedBlockedUsersNum = TOptional<uint32_t>())
		: HelperParams(MoveTemp(InHelperParams))
		, ExpectedBlockedUsersNum(InExpectedBlockedUsersNum)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FGetBlockedUsersHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SocialInterface = Services->GetSocialInterface();
		REQUIRE(SocialInterface);

		ResultType Result = SocialInterface->GetBlockedUsers(MoveTemp(*HelperParams.OpParams));

		if (HelperParams.ExpectedError.IsSet())
		{
			REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
		}
		else if (ExpectedBlockedUsersNum.IsSet())
		{
			CHECK(Result.GetOkValue().BlockedUsers.Num() == ExpectedBlockedUsersNum.GetValue());
		}
		else
		{
			CHECK(Result.GetOkValue().BlockedUsers.IsEmpty());
		}
		Promise->SetValue(true);
	}

protected:
	FHelperParams HelperParams;
	TOptional<uint32_t> ExpectedBlockedUsersNum = TOptional<uint32_t>();
	UE::Online::ISocialPtr SocialInterface = nullptr;
};