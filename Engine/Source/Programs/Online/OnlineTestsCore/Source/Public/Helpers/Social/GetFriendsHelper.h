// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SocialCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FGetFriendsHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FGetFriends::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FGetFriends>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FGetFriendsHelper(FHelperParams&& InHelperParams, const TOptional<uint32_t> InExpectedFriendsNum = TOptional<uint32_t>())
		: HelperParams(MoveTemp(InHelperParams))
		, ExpectedFriendsNum(InExpectedFriendsNum)
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FGetFriendsHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SocialInterface = Services->GetSocialInterface();
		REQUIRE(SocialInterface);

		ResultType Result = SocialInterface->GetFriends(MoveTemp(*HelperParams.OpParams));
		
		if (HelperParams.ExpectedError.IsSet())
		{
			REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
		}
		else if(ExpectedFriendsNum.IsSet())
		{
			CHECK(Result.GetOkValue().Friends.Num() == ExpectedFriendsNum.GetValue());
		}
		else
		{
			CHECK(Result.GetOkValue().Friends.IsEmpty());
		}
		Promise->SetValue(true);
	}

protected:
	FHelperParams HelperParams;
	TOptional<uint32_t> ExpectedFriendsNum = TOptional<uint32_t>();
	UE::Online::ISocialPtr SocialInterface = nullptr;
};