// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/SocialCommon.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

struct FQueryFriendsHelper : public FAsyncTestStep
{
	using ParamsType = UE::Online::FQueryFriends::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FQueryFriends>;

	struct FHelperParams
	{
		ParamsType* OpParams = nullptr;
		TOptional<ResultType> ExpectedError;
	};

	FQueryFriendsHelper(FHelperParams&& InHelperParams)
		: HelperParams(MoveTemp(InHelperParams))
	{
		REQUIRE(HelperParams.OpParams);
		REQUIRE((!HelperParams.ExpectedError.IsSet() || HelperParams.ExpectedError->IsError()));
	}

	virtual ~FQueryFriendsHelper() = default;

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		SocialInterface = Services->GetSocialInterface();
		REQUIRE(SocialInterface);

		SocialInterface->QueryFriends(MoveTemp(*HelperParams.OpParams))
			.OnComplete([this, Promise = MoveTemp(Promise)](const ResultType& Result)
				{
					if (!HelperParams.ExpectedError.IsSet())
					{
						REQUIRE(Result.IsOk());
					}
					else
					{
						REQUIRE_OP_EQ(Result, HelperParams.ExpectedError->GetErrorValue());
					}
					Promise->SetValue(true);
				});
	}

protected:
	FHelperParams HelperParams;
	UE::Online::ISocialPtr SocialInterface = nullptr;
};