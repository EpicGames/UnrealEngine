// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/Stats.h"
#include "OnlineCatchHelper.h"

namespace UE::Online
{

inline void UpdateStats_Fixture(FAsyncLambdaResult Promise, const IOnlineServicesPtr& Services, const FAccountId& AccountId,  const TArray<FUserStats>& UpdateUsersStats)
{
	Services->GetStatsInterface()->UpdateStats({AccountId,  UpdateUsersStats}).OnComplete([Promise=MoveTemp(Promise)](const TOnlineResult<UE::Online::FUpdateStats>& Result) mutable
	{
		CHECK_OP(Result);
		Promise->SetValue(true);
	});
}

}
