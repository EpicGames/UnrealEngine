// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once
 
#include "TestDriver.h"
 
struct FTickForTime : public FTestPipeline::FStep
{
public:
	FTickForTime(const FTimespan InTimeToWaitMilliseconds)
		: TimeToWaitMilliseconds(InTimeToWaitMilliseconds)
	{
	}
 
	virtual EContinuance Tick(const IOnlineServicesPtr& OnlineSubsystem) override
	{
		double Now = FPlatformTime::Seconds();
 
		if (!bHasSetStartTime)
		{
			bHasSetStartTime = true;
			StartTime = Now;
			return EContinuance::ContinueStepping;
		}
 
		if (FTimespan::FromSeconds(Now - StartTime).GetTicks() >= TimeToWaitMilliseconds.GetTicks())
		{
			return EContinuance::Done;
		}
 
		return EContinuance::ContinueStepping;
	}
 
	virtual ~FTickForTime() = default;
 
protected:
	double StartTime;
	bool bHasSetStartTime = false;
	FTimespan TimeToWaitMilliseconds;
};