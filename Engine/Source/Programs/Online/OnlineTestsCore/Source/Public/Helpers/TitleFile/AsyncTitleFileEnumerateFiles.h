// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/OnlineAsyncOpHandle.h"
#include "Online/TitleFile.h"
#include "AsyncTestStep.h"

class FAsyncTitleFileEnumerateFiles : public FAsyncTestStep
{
public:
	using ParamsType = UE::Online::FTitleFileEnumerateFiles::Params;
	using OpResultType = UE::Online::FTitleFileEnumerateFiles::Result;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FTitleFileEnumerateFiles>;

	FAsyncTitleFileEnumerateFiles(ParamsType&& InParams)
		: Params(MoveTemp(InParams))
		, ExpectedResult(OpResultType())
	{
	}

	template <typename ResultParam>
	FAsyncTitleFileEnumerateFiles(ParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::ITitleFilePtr TitleFile = Services->GetTitleFileInterface();
		REQUIRE(TitleFile != nullptr);
		UE::Online::TOnlineAsyncOpHandle<UE::Online::FTitleFileEnumerateFiles> Op = TitleFile->EnumerateFiles(MoveTemp(Params));
		Op.OnComplete([this, Promise](const ResultType Result) mutable
		{
			CAPTURE(Result);
			REQUIRE(Result.IsOk() == ExpectedResult.IsOk());
			if (Result.IsError())
			{
				REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
			}

			Promise->SetValue(true);
		});
	}

protected:
	ParamsType Params;
	ResultType ExpectedResult;
};