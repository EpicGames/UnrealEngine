// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/TitleFile.h"
#include "AsyncTestStep.h"

class FAsyncTitleFileGetEnumeratedFiles : public FAsyncTestStep
{
public:
	using ParamsType = UE::Online::FTitleFileGetEnumeratedFiles::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FTitleFileGetEnumeratedFiles>;
	
	template <typename ResultParam>
	FAsyncTitleFileGetEnumeratedFiles(ParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::ITitleFilePtr TitleFile = Services->GetTitleFileInterface();
		REQUIRE(TitleFile != nullptr);
		const ResultType Result = TitleFile->GetEnumeratedFiles(MoveTemp(Params));
		
		REQUIRE(Result.IsOk() == ExpectedResult.IsOk());
		if (Result.IsOk())
		{
			REQUIRE(Result.GetOkValue().Filenames == ExpectedResult.GetOkValue().Filenames);
		}
		else
		{
			REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
		}

		Promise->SetValue(true);
	}

	ParamsType Params;
	ResultType ExpectedResult;
};