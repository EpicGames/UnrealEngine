// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/OnlineAsyncOpHandle.h"
#include "Online/TitleFile.h"
#include "AsyncTestStep.h"

class FAsyncTitleFileReadFile : public FAsyncTestStep
{
public:
	using ParamsType = UE::Online::FTitleFileReadFile::Params;
	using ResultType = UE::Online::TOnlineResult<UE::Online::FTitleFileReadFile>;

	template <typename ResultParam>
	FAsyncTitleFileReadFile(ParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::ITitleFilePtr TitleFile = Services->GetTitleFileInterface();
		REQUIRE(TitleFile != nullptr);
		UE::Online::TOnlineAsyncOpHandle<UE::Online::FTitleFileReadFile> Op = TitleFile->ReadFile(MoveTemp(Params));
		Op.OnComplete([this, Promise](const ResultType Result) mutable
		{
			REQUIRE(Result.IsOk() == ExpectedResult.IsOk());
			if (Result.IsOk())
			{
				REQUIRE(*Result.GetOkValue().FileContents == *ExpectedResult.GetOkValue().FileContents);
			}
			else
			{
				REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
			}
			Promise->SetValue(true);
		});
	}

protected:
	ParamsType Params;
	ResultType ExpectedResult;
};