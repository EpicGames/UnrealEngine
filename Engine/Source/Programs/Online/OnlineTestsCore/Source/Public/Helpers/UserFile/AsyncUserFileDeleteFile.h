// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/UserFile.h"
#include "AsyncTestStep.h"

class FAsyncUserFileDeleteFile : public FAsyncTestStep
{
public:
	using OpType = UE::Online::FUserFileDeleteFile;
	using OpParamsType = OpType::Params;
	using OpResultType = OpType::Result;
	using OnlineResultType = UE::Online::TOnlineResult<OpType>;

	FAsyncUserFileDeleteFile(OpParamsType&& InParams)
		: Params(MoveTemp(InParams))
	{
		ExpectedResults.Emplace(OpResultType());
	}

	template <typename ResultParam>
	FAsyncUserFileDeleteFile(OpParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
	{
		ExpectedResults.Emplace(MoveTemp(InExpectedResult));
	}

	template <typename ResultParam>
	FAsyncUserFileDeleteFile(OpParamsType&& InParams, TArray<ResultParam>&& InExpectedResults)
		: Params(MoveTemp(InParams))
		, ExpectedResults(MoveTemp(InExpectedResults))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::IUserFilePtr UserFile = Services->GetUserFileInterface();
		REQUIRE(UserFile != nullptr);
		UE::Online::TOnlineAsyncOpHandle<OpType> Op = UserFile->DeleteFile(MoveTemp(Params));

		Op.OnComplete([this, Promise](const OnlineResultType Result) mutable
		{
			bool bPass = false;
			// Loop once to determine if we pass or not
			for (const OnlineResultType& ExpectedResult : ExpectedResults)
			{
				if (Result.IsOk() == ExpectedResult.IsOk()
					&& (!Result.IsError() || Result.GetErrorValue() == ExpectedResult.GetErrorValue()))
				{
					bPass = true;
					break;
				}
			}
			// If we didn't pass, loop again to fire a bunch of errors.
			if (!bPass)
			{
				for (const OnlineResultType& ExpectedResult : ExpectedResults)
				{
					CHECK(Result.IsOk() == ExpectedResult.IsOk());
					if (Result.IsError())
					{
						UE::Online::FOnlineError Error = ExpectedResult.IsError() ? ExpectedResult.GetErrorValue() : UE::Online::FOnlineError(UE::Online::Errors::Unknown());
						CHECK(Result.GetErrorValue() == Error);
					}
				}
			}

			Promise->SetValue(true);
		});
	}

protected:
	OpParamsType Params;
	TArray<OnlineResultType> ExpectedResults;
};