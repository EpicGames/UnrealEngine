// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/OnlineAsyncOpHandle.h"
#include "Online/UserFile.h"
#include "AsyncTestStep.h"
#include "OnlineCatchHelper.h"

class FAsyncUserFileEnumerateFiles : public FAsyncTestStep
{
public:
	using OpType = UE::Online::FUserFileEnumerateFiles;
	using OpParamsType = OpType::Params;
	using OpResultType = OpType::Result;
	using OnlineResultType = UE::Online::TOnlineResult<OpType>;

	FAsyncUserFileEnumerateFiles(OpParamsType&& InParams)
		: Params(MoveTemp(InParams))
		, ExpectedResult(OpResultType())
	{
	}

	template <typename ResultParam>
	FAsyncUserFileEnumerateFiles(OpParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::IUserFilePtr UserFile = Services->GetUserFileInterface();
		REQUIRE(UserFile != nullptr);
		UE::Online::TOnlineAsyncOpHandle<OpType> Op = UserFile->EnumerateFiles(MoveTemp(Params));

		Op.OnComplete([this, Promise](const OnlineResultType Result) mutable
		{
			if(ExpectedResult.IsOk())
			{
				CHECK_OP(Result);
			}
			else
			{
				REQUIRE(!Result.IsOk());
				if (Result.IsError())
				{
					REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
				}
			}

			Promise->SetValue(true);
		});
	}

protected:
	OpParamsType Params;
	OnlineResultType ExpectedResult;
};