// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/UserFile.h"
#include "AsyncTestStep.h"

class FAsyncUserFileGetEnumeratedFiles : public FAsyncTestStep
{
public:
	using OpType = UE::Online::FUserFileGetEnumeratedFiles;
	using OpParamsType = OpType::Params;
	using OpResultType = OpType::Result;
	using OnlineResultType = UE::Online::TOnlineResult<OpType>;

	FAsyncUserFileGetEnumeratedFiles(OpParamsType&& InParams)
		: Params(MoveTemp(InParams))
		, ExpectedResult(OpResultType())
	{
	}

	template <typename ResultParam>
	FAsyncUserFileGetEnumeratedFiles(OpParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::IUserFilePtr UserFile = Services->GetUserFileInterface();
		REQUIRE(UserFile != nullptr);
		const OnlineResultType Result = UserFile->GetEnumeratedFiles(MoveTemp(Params));

		REQUIRE(Result.IsOk() == ExpectedResult.IsOk());
		if (Result.IsOk())
		{
			REQUIRE(Result.GetOkValue().Filenames == ExpectedResult.GetOkValue().Filenames);
		}
		else
		{
			REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
		}

		Promise->SetValue(true);
	}

	OpParamsType Params;
	OnlineResultType ExpectedResult;
};