// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/UserFile.h"
#include "AsyncTestStep.h"

class FAsyncUserFileReadFile : public FAsyncTestStep
{
public:
	using OpType = UE::Online::FUserFileReadFile;
	using OpParamsType = OpType::Params;
	using OpResultType = OpType::Result;
	using OnlineResultType = UE::Online::TOnlineResult<OpType>;

	FAsyncUserFileReadFile(OpParamsType&& InParams)
		: Params(MoveTemp(InParams))
		, ExpectedResult(OpResultType())
	{
	}

	template <typename ResultParam>
	FAsyncUserFileReadFile(OpParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::IUserFilePtr UserFile = Services->GetUserFileInterface();
		REQUIRE(UserFile != nullptr);
		UE::Online::TOnlineAsyncOpHandle<OpType> Op = UserFile->ReadFile(MoveTemp(Params));

		Op.OnComplete([this, Promise](const OnlineResultType Result) mutable
		{
			REQUIRE(Result.IsOk() == ExpectedResult.IsOk());
			if (Result.IsOk())
			{
				REQUIRE(*Result.GetOkValue().FileContents == *ExpectedResult.GetOkValue().FileContents);
			}
			else
			{
				REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
			}

			Promise->SetValue(true);
		});
	}

protected:
	OpParamsType Params;
	OnlineResultType ExpectedResult;
};