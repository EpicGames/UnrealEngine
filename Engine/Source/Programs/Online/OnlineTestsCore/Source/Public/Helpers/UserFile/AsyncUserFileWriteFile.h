// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/UserFile.h"
#include "AsyncTestStep.h"

class FAsyncUserFileWriteFile : public FAsyncTestStep
{
public:
	using OpType = UE::Online::FUserFileWriteFile;
	using OpParamsType = OpType::Params;
	using OpResultType = OpType::Result;
	using OnlineResultType = UE::Online::TOnlineResult<OpType>;

	FAsyncUserFileWriteFile(OpParamsType&& InParams)
		: Params(MoveTemp(InParams))
		, ExpectedResult(OpResultType())
	{
	}

	template <typename ResultParam>
	FAsyncUserFileWriteFile(OpParamsType&& InParams, ResultParam&& InExpectedResult)
		: Params(MoveTemp(InParams))
		, ExpectedResult(MoveTemp(InExpectedResult))
	{
	}

	virtual void Run(FAsyncStepResult Promise, const IOnlineServicesPtr& Services) override
	{
		REQUIRE(Services != nullptr);
		UE::Online::IUserFilePtr UserFile = Services->GetUserFileInterface();
		REQUIRE(UserFile != nullptr);
		UE::Online::TOnlineAsyncOpHandle<OpType> Op = UserFile->WriteFile(MoveTemp(Params));

		Op.OnComplete([this, Promise](const OnlineResultType& Result) mutable
		{
			REQUIRE(Result.IsOk() == ExpectedResult.IsOk());
			if (Result.IsError())
			{
				REQUIRE(Result.GetErrorValue() == ExpectedResult.GetErrorValue());
			}
			
			if (FuturePtr && !FuturePtr->IsReady())
			{
				Promise->SetValue(true);
			}
		});
	}

protected:
	OpParamsType Params;
	OnlineResultType ExpectedResult;
};