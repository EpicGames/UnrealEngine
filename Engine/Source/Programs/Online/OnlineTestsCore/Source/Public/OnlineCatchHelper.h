// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TestHarness.h"
#include "TestDriver.h"
#include "CoreMinimal.h"
#include "OnlineCatchStringMakers.h"
#include "Interfaces/OnlineIdentityInterface.h"
#include "Online/AuthCommon.h"
#include "Online/OnlineAsyncOp.h"
#include "Online/OnlineError.h"
#include "Online/OnlineErrorDefinitions.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineResult.h"
#include "Online/OnlineServicesCommon.h"

#include <catch2/catch_test_case_info.hpp>
#include <catch2/catch_test_macros.hpp>
#include <catch2/interfaces/catch_interfaces_registry_hub.hpp>
#include <catch2/catch_active_test.hpp>
#include <string>

using namespace UE::Online;

class OnlineTestBase : public Catch::ITestInvoker
{
public:
	void ConstructInternal(FString ServiceName, UE::Online::EOnlineServices ServiceType);

	// must be public so that Catch2::Detail::unique_ptr can call the destructor
	virtual ~OnlineTestBase();

	/* Loads all necessary services for the current test run */
	static void LoadServiceModules();

	/* Unloads all necessary services for the current test run */
	static void UnloadServiceModules();

#if ONLINETESTS_USEEXTERNAUTH
	/* Returning accounts to the pool */
	static bool ReturnAccounts(int32 TestAccountIndex = -1);
#endif

	FTestPipeline& GetLoginPipeline(std::initializer_list<std::reference_wrapper<FAccountId>> AccountIds) const;

	FTestPipeline& GetLoginPipeline(uint32 UserNumToLogin, std::initializer_list<std::reference_wrapper<FAccountId>> AccountIds) const;

	typedef UE::Online::FAccountId FAccountId;
public:

	/* Attempts to assign OutAccountId to what LocalUserId is logged in as. */
	void AssignLoginUsers(int32 LocalUserId, FAccountId& OutAccountId) const;

#if ONLINETESTS_USEEXTERNAUTH
	TArray<FString> GetExternalAuthAccountIds(int32 TestAccountIndex) const;
#endif

	FTestPipeline& GetPipeline() const;
protected:

	OnlineTestBase();

	/* Helper function to delete accounts from TestDataService */
	bool DeleteAccounts(int32 TestAccountIndex = -1) const;

	/* Helper function to destroy the current OnlineService module which stores some state that maybe needs to be reset */
	void DestroyCurrentServiceModule() const;

	/* Proxy function to the DestroyCurrentServiceModule() */
	bool ResetAccountStatus(int32 TestAccountIndex = -1) const;

	/* Returns the name of the service we're currently testing */
	FString GetService() const;
	UE::Online::EOnlineServices GetServiceType() const;
	IOnlineServicesPtr GetServices() const;

// If you define a custom credentials path you must define these in your own Auth.cpp file
#if ONLINETESTS_USEEXTERNAUTH
	// This will provide a parameters to pass to the OSS Auth interface.
	TArray<UE::Online::FAuthLogin::Params> CustomCredentials(int32 LocalUserNum, int32 NumUsers) const;

	// When called should "reset" accounts to default state for Stats, Achivements, Lobbies and other
	// stateful interfaces to work.
	bool CustomResetAccounts(int32 TestAccountIndex) const;

	// When called will *remove* the account from the underlying service.
	// This is generally called to test new-user login or first-time setup.
	// To be called rare-ly.
	bool CustomDeleteAccounts(int32 TestAccountIndex) const;

#endif
	TArray<UE::Online::FAuthLogin::Params> GetIniCredentials(int32 TestAccountIndex) const;

	TArray<UE::Online::FAuthLogin::Params> GetCredentials(int32 TestAccountIndex, int32 NumUsers) const;

	/* Returns the ini login category name for the configured service */
	FString GetLoginCredentialCategory() const;

	void RunToCompletion(bool bLogout = true) const;

	/* ITestInvoker */
	virtual void invoke() const override = 0;

private:
	FString Tags;
	FString Service;
	UE::Online::EOnlineServices ServiceType;
	// Catch's ITestInvoker is a const interface but we'll be changing stuff (emplacing steps into the driver, setting flags, etc.) so we make these mutable
	mutable FTestDriver Driver;
	mutable TSharedPtr<FTestPipeline> Pipeline;
	mutable uint32 NumLocalUsers = -1;
	mutable uint32 TestAccountID = -1;
};

typedef OnlineTestBase* (*OnlineTestConstructor)();

class OnlineAutoReg
{
public:
	struct FReportingSkippableTags
	{
		TArray<FString> MayFailTags;
		TArray<FString> ShouldFailTags;
		TArray<FString> DisableTestTags;
	};

	/*
	* Helper function that calls CheckAllTagsIsIn(const TArray<FString>&, const TArray<FString>&);
	* 
	* @param  TestTags			The array of test tags we wish to test against.
	* @param  RawTagString		Comma seperated string of elemnets we wish to convert to an array.
	* 
	* @return true if all elements of RawTagString prased as an comma sperated array is in TestTags.
	*/
	static bool CheckAllTagsIsIn(const TArray<FString>& TestTags, const FString& RawTagString);
	

	/**
	 * Checks if every element of InputTags is in TestTags.
	 *
	 * @param  TestTags       The array of test tags we wish to test against.
	 * @param  InputTags	  The array of tags we wish to see if all elements of are present in TestTags.
	 *
	 * @return  true if all elements of InputTags is in TestTags.
	 */
	static bool CheckAllTagsIsIn(const TArray<FString>& TestTags, const TArray<FString>& InputTags);
	
	static FString GenerateTags(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, const TCHAR* InTag);
	static bool ShouldDisableTest(const FString& ServiceName, const FReportingSkippableTags& SkippableTags, const TCHAR* InTag);
	static bool ShouldSkipTest(const FString& TagsToCheck);
	static void CheckRunningTestSkipOnTags();

	// This code is kept identical to Catch internals so that there is as little deviation from OSS_TESTS and Online_OSS_TESTS as possible
	OnlineAutoReg(OnlineTestConstructor TestCtor, Catch::SourceLineInfo LineInfo, const char* Name, const char* Tags, const char* AddlOnlineInfo);

	struct FApplicableServicesConfig
	{
		FString Tag;
		EOnlineServices ServicesType;
		TArray<FString> ModulesToLoad;
	};

	static TArray<FApplicableServicesConfig> GetApplicableServices();
};

TArray<TFunction<void()>>* GetGlobalInitalizers();

// INTERNAL_CATCH_UNIQUE_NAME is what Catch uses to translate the name of the test into a unique identifier without any spaces

#define INTERNAL_ONLINE_TEST_CASE_NAMED(RegName, Name, Tags, ...)\
namespace {\
class UE_JOIN(OnlineTest_,RegName) : public OnlineTestBase {\
protected:\
	virtual void invoke() const override;\
};\
	OnlineTestBase* UE_JOIN(Construct_,RegName) (){\
		return new UE_JOIN(OnlineTest_,RegName) ();\
	}\
	OnlineAutoReg RegName( UE_JOIN(&Construct_,RegName) , CATCH_INTERNAL_LINEINFO , Name, Tags, "");\
}\
void UE_JOIN(OnlineTest_,RegName)::invoke() const\

#define ONLINE_TEST_CASE(Name, Tags, ...) \
	INTERNAL_ONLINE_TEST_CASE_NAMED(INTERNAL_CATCH_UNIQUE_NAME(OnlineRegistrar), Name, Tags, __VA_ARGS__)

// Helpers for checking+capturing at the same time

#define CHECK_OP(Op)\
	CAPTURE(Op);\
	CHECK(Op.IsOk());

// also passes if the op is a success
#define CHECK_OP_EQ(Op, Arg)\
	CAPTURE(Op);\
	CHECK((Op.IsOk() || Op.GetErrorValue() == Arg));


#define CHECK_STR(Str)\
	CAPTURE(Str);\
	CHECK(!Str.IsEmpty());

#define CHECK_STR_EMPTY(Str)\
	CAPTURE(Str);\
	CHECK(Str.IsEmpty());


#define REQUIRE_OP(Op)\
	CAPTURE(Op);\
	REQUIRE(Op.IsOk());

// also passes if the op is a success
#define REQUIRE_OP_EQ(Op, Arg)\
	CAPTURE(Op);\
	REQUIRE((Op.IsOk() || Op.GetErrorValue() == Arg));


#define REQUIRE_STR(Str)\
	CAPTURE(Str);\
	REQUIRE(!Str.IsEmpty());

#define REQUIRE_STR_EMPTY(Str)\
	CAPTURE(Str);\
	REQUIRE(Str.IsEmpty());