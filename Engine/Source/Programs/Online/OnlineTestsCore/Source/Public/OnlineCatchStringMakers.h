// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TestHarness.h"
#include "Online/OnlineResult.h"

namespace UE::Online
{
	class FOnlineError;
}

namespace Catch
{
	template<>
	struct StringMaker<UE::Online::FOnlineError>
	{
		static std::string convert(UE::Online::FOnlineError const& Error);
	};

	template <typename T>
	struct StringMaker<UE::Online::TOnlineResult<T>>
	{
		static std::string convert(UE::Online::TOnlineResult<T> const& Result)
		{
			return StringMaker<FString>::convert(ToLogString(Result));
		}
	};
}

